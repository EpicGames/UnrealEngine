[CoreRedirects]

+EnumRedirects=(OldName="FZoneShapePointType" ValueChanges=(("FZoneShapePointType::LaneSegment","FZoneShapePointType::LaneProfile")))

+StructRedirects=(OldName="ZoneLaneTemplate",NewName="/script/ZoneGraph.ZoneLaneProfile")
+StructRedirects=(OldName="ZoneLaneTemplateRef",NewName="/script/ZoneGraph.ZoneLaneProfileRef")

+PropertyRedirects=(OldName="ZoneGraphSettings.LaneTemplates",NewName="LaneProfiles")
+PropertyRedirects=(OldName="ZoneShapeComponent.LaneTemplate",NewName="LaneProfile")
+PropertyRedirects=(OldName="ZoneShapeComponent.PerPointLaneTemplates",NewName="PerPointLaneProfiles")
+PropertyRedirects=(OldName="ZoneShapePoint.LaneTemplate",NewName="LaneProfile")

+PropertyRedirects=(OldName="ZoneShapePoint.InControlPoint",NewName="InControlPoint_DEPRECATED")
+PropertyRedirects=(OldName="ZoneShapePoint.OutControlPoint",NewName="OutControlPoint_DEPRECATED")

+PropertyRedirects=(OldName="ZoneGraphTagFilter.IncludeTags",NewName="AnyTags")
+PropertyRedirects=(OldName="ZoneGraphTagFilter.ExcludeTags",NewName="NotTags")