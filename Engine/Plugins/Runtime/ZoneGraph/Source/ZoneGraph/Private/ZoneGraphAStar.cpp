// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZoneGraphAStar.h"
#include "ZoneGraphTypes.h"
#include "ZoneGraphQuery.h"

int32 FZoneGraphAStarWrapper::GetNeighbourCountV2(const FZoneGraphAStarNode& Node) const
{
	// @todo: Make a special case to allow picking side neighbours when starting in an intersection.
	//  Use ZoneData.Tags to identify where this should apply and return the correct neighbour count here.

	// Return max link count, we'll return invalid link for the ones that we do not want to traverse.
	const FZoneLaneData& Lane = ZoneGraph.Lanes[Node.NodeRef];
	return Lane.GetLinkCount();
}

FZoneGraphAStarWrapper::FNodeRef FZoneGraphAStarWrapper::GetNeighbour(const FZoneGraphAStarNode& Node, const int32 NeighbourIndex) const
{
	// @todo: Make a special case to allow picking side neighbours when starting in an intersections (use ZoneData.Tags to identify where this should apply).

	const FZoneLaneData& Lane = ZoneGraph.Lanes[Node.NodeRef];
	check(NeighbourIndex < Lane.GetLinkCount());

	const int32 LinkIndex = Lane.LinksBegin + NeighbourIndex;
	const FZoneLaneLinkData& Link = ZoneGraph.LaneLinks[Lane.LinksBegin + NeighbourIndex];


	// Allow to pick adjacent lanes if start node.
	if (Node.IsStartOrIsEnd())
	{
		// Allow to pick left/right adjacent flags at start/end.
		if (Link.Type == EZoneLaneLinkType::Adjacent && Link.HasFlags(EZoneLaneLinkFlags::Left | EZoneLaneLinkFlags::Right))
		{
			return Link.DestLaneIndex;
		}
	}

	// Normally allow only outgoing lanes
	if (Link.Type == EZoneLaneLinkType::Outgoing)
	{
		return Link.DestLaneIndex;
	}

	return INDEX_NONE;
}
