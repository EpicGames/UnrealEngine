// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZoneGraphBVTree.h"
#include "Math/Box.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ZoneGraphBVTree)

namespace UE::ZoneGraph::BVTree
{
	// Returns longest axis of the node bounds.
	static int32 GetLongestAxis(const FZoneGraphBVNode& Node)
	{
		const uint16 DimX = Node.MaxX - Node.MinX;
		const uint16 DimY = Node.MaxY - Node.MinY;
		const uint16 DimZ = Node.MaxZ - Node.MinZ;

		if (DimX > DimY && DimX > DimZ)
		{
			return 0;
		}
		else if (DimY > DimZ)
		{
			return 1;
		}
		return 2;
	}

	// Calculate bounds of items in range [BeginIndex, EndIndex] (EndIndex non-inclusive).
	static FZoneGraphBVNode CalcNodeBounds(const TArray<FZoneGraphBVNode>& Items, const int32 BeginIndex, const int32 EndIndex)
	{
		check(EndIndex > BeginIndex);

		FZoneGraphBVNode Result = Items[BeginIndex];
		
		for (int32 Index = BeginIndex + 1; Index < EndIndex; ++Index)
		{
			const FZoneGraphBVNode& Node = Items[Index];
			Result.MinX = FMath::Min(Result.MinX, Node.MinX);
			Result.MinY = FMath::Min(Result.MinY, Node.MinY);
			Result.MinZ = FMath::Min(Result.MinZ, Node.MinZ);
			Result.MaxX = FMath::Max(Result.MaxX, Node.MaxX);
			Result.MaxY = FMath::Max(Result.MaxY, Node.MaxY);
			Result.MaxZ = FMath::Max(Result.MaxZ, Node.MaxZ);
		}
		
		return Result;
	}

	// Creates subtree of nodes in range [BeginIndex, EndIndex] (EndIndex non-inclusive), by sorting the nodes along the longest axis of all items, and splitting them equally (in count) in two subtrees.
	static void Subdivide(TArray<FZoneGraphBVNode>& Items, const int32 BeginIndex, const int32 EndIndex, TArray<FZoneGraphBVNode>& OutNodes)
	{
		struct FAxisSort
		{
			const int Axis;
			FAxisSort(const int InAxis) : Axis(InAxis) {}

			bool operator()(const FZoneGraphBVNode& A, const FZoneGraphBVNode& B) const
			{
				// Pointer trick stolen from FVector.
				const uint16 ValueA = (&A.MinX)[Axis];
				const uint16 ValueB = (&B.MinX)[Axis];
				return ValueA < ValueB;
			}
		};

		const int32 Count = EndIndex - BeginIndex;
		const int32 CurrentNodeIndex = OutNodes.Num();

		FZoneGraphBVNode& Node = OutNodes.AddDefaulted_GetRef();

		if (Count == 1)
		{
			// Leaf node
			Node = Items[BeginIndex];
		}
		else
		{
			// Needs splitting
			Node = CalcNodeBounds(Items, BeginIndex, EndIndex);

			const int32 Axis = GetLongestAxis(Node);
			Algo::Sort(MakeArrayView(Items.GetData() + BeginIndex, Count), FAxisSort(Axis));
			
			const int32 SplitIndex = BeginIndex + Count / 2;

			// Left
			Subdivide(Items, BeginIndex, SplitIndex, OutNodes);
			// Right
			Subdivide(Items, SplitIndex, EndIndex, OutNodes);

			// Negative index means skip the subtree to next sibling.
			const int32 NextSiblingIndex = OutNodes.Num() - CurrentNodeIndex;
			Node.Index = -NextSiblingIndex;
		}
	}
	
} // UE::ZoneGraph::BVTree


void FZoneGraphBVTree::Build(TStridedView<const FBox> Boxes)
{
	// Reset current state
	Nodes.Reset();
	Origin = FVector::ZeroVector;
	QuantizationScale = 0.0f;

	if (Boxes.Num() == 0)
	{
		return;
	}
	
	// Calculate quantization values from the bounds containing all the boxes.
	FBox TotalBounds(ForceInit);
	for (const FBox& Box : Boxes)
	{
		TotalBounds += Box;
	}

	const FVector BoxSize = TotalBounds.GetSize();
	const float MaxDimension = FMath::Max(1.0f, BoxSize.GetMax());
	QuantizationScale = MaxQuantizedCoord / MaxDimension;
	Origin = TotalBounds.Min;

	// Quantize boxes
	TArray<FZoneGraphBVNode> Items;
	int32 Index = 0;
	for (const FBox& Box : Boxes)
	{
		FZoneGraphBVNode& Item = Items.AddDefaulted_GetRef();
		Item = CalcNodeBounds(Box);
		Item.Index = Index++;
	}

	// Build tree
	Nodes.Reserve(Items.Num() * 2 - 1);
	UE::ZoneGraph::BVTree::Subdivide(Items, 0, Items.Num(), Nodes);
}

void FZoneGraphBVTree::Query(const FBox& Bounds, TArray<int32>& OutItems) const
{
	OutItems.Reset();
	Query(Bounds, [&OutItems](const FZoneGraphBVNode& Node)
	{
		OutItems.Add(Node.Index);
	});
}
