// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZoneGraphBuilder.h"
#include "ZoneGraphTypes.h"
#include "ZoneGraphDelegates.h"
#include "ZoneGraphData.h"
#include "ZoneShapeComponent.h"
#include "ZoneShapeUtilities.h"
#include "ZoneGraphSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ZoneGraphBuilder)

namespace UE::ZoneGraph::Internal
{
	
inline bool InRange(const double Value, const double Min, const double Max)
{
	return Value >= Min && Value <= Max;
}
	
} // UE::ZoneGraph::Internal

FZoneGraphBuilder::FZoneGraphBuilder()
	: HashGrid(500.0f)	// 5m grid, TODO: make configurable
{
#if WITH_EDITOR
	OnTagsChangedHandle = UE::ZoneGraphDelegates::OnZoneGraphTagsChanged.AddRaw(this, &FZoneGraphBuilder::RequestRebuild);
	OnLaneProfileChangedHandle = UE::ZoneGraphDelegates::OnZoneGraphLaneProfileChanged.AddRaw(this, &FZoneGraphBuilder::OnLaneProfileChanged);
	OnBuildSettingsChangedHandle = UE::ZoneGraphDelegates::OnZoneGraphBuildSettingsChanged.AddRaw(this, &FZoneGraphBuilder::RequestRebuild);
#endif

}

FZoneGraphBuilder::~FZoneGraphBuilder()
{
#if WITH_EDITOR
	UE::ZoneGraphDelegates::OnZoneGraphTagsChanged.Remove(OnTagsChangedHandle);
	UE::ZoneGraphDelegates::OnZoneGraphLaneProfileChanged.Remove(OnLaneProfileChangedHandle);
	UE::ZoneGraphDelegates::OnZoneGraphBuildSettingsChanged.Remove(OnBuildSettingsChangedHandle);
#endif
}

void FZoneGraphBuilder::OnLaneProfileChanged(const FZoneLaneProfileRef& ChangedLaneProfileRef)
{
	bSkipHashCheck = true;
	bIsDirty = true;
}

void FZoneGraphBuilder::RequestRebuild()
{
	bSkipHashCheck = true;
	bIsDirty = true;
}

void FZoneGraphBuilder::RegisterZoneShapeComponent(UZoneShapeComponent& ShapeComp)
{
	// TODO: Move the builder into editor. This call is guarded editor only at call site already.
#if WITH_EDITOR
	// TODO: we could potentially separate out automatic building logic, and use object iterator to get all relevant data instead.
	// Add to list
	int32 Index = ShapeComponentsFreeList.IsEmpty() ? ShapeComponents.AddDefaulted() : ShapeComponentsFreeList.Pop();
	FZoneGraphBuilderRegisteredComponent& Registered = ShapeComponents[Index];
	Registered.Component = &ShapeComp;
	// Add to grid
	const FBoxSphereBounds Bounds = ShapeComp.CalcBounds(ShapeComp.GetComponentTransform());
	Registered.CellLoc = HashGrid.Add(uint32(Index), Bounds.GetBox());
	// Add to index lookup
	ShapeComponentToIndex.Add(&ShapeComp, Index);
	// Compute shape hash, used to detect if the shape has changed.
	Registered.ShapeHash = ShapeComp.GetShapeHash();

	bIsDirty = true;
#endif
}

void FZoneGraphBuilder::UnregisterZoneShapeComponent(UZoneShapeComponent& ShapeComp)
{
	// TODO: Move the builder into editor. This call is guarded editor only at call site already.
#if WITH_EDITOR
	if (int32* Index = ShapeComponentToIndex.Find(&ShapeComp))
	{
		check(ShapeComponents.IsValidIndex(*Index));
		FZoneGraphBuilderRegisteredComponent& Registered = ShapeComponents[*Index];
		// Remove from grid
		HashGrid.Remove(uint32(*Index), Registered.CellLoc);
		// Remove from index lookup
		ShapeComponentToIndex.Remove(Registered.Component);
		// Remove from list
		Registered.Component = nullptr;
		ShapeComponentsFreeList.Add(*Index);
	}

	bIsDirty = true;
#endif
}

void FZoneGraphBuilder::OnZoneShapeComponentChanged(UZoneShapeComponent& ShapeComp)
{
	// TODO: Move the builder into editor. This call is guarded editor only at call site already.
#if WITH_EDITOR
	if (int32* Index = ShapeComponentToIndex.Find(&ShapeComp))
	{
		check(ShapeComponents.IsValidIndex(*Index));
		FZoneGraphBuilderRegisteredComponent& Registered = ShapeComponents[*Index];
		if (Registered.Component)
		{
			const FBoxSphereBounds NewBounds = Registered.Component->CalcBounds(ShapeComp.GetComponentTransform());
			Registered.CellLoc = HashGrid.Move(uint32(*Index), Registered.CellLoc, NewBounds.GetBox());
			Registered.ShapeHash = Registered.Component->GetShapeHash();
		}
	}

	bIsDirty = true;
#endif
}

uint32 FZoneGraphBuilder::CalculateCombinedShapeHash(const AZoneGraphData& ZoneGraphData) const
{
	// TODO: Move the builder into editor. Hashing is editor only.
#if WITH_EDITOR
	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
	ULevel* CurrentLevel = ZoneGraphData.GetLevel();
	check(ZoneGraphSettings);
	check(CurrentLevel);

	// Start with settings
	uint32 CombinedHash = ZoneGraphSettings->GetBuildHash();

	// Combine all included component hashes
	for (const FZoneGraphBuilderRegisteredComponent& Registered : ShapeComponents)
	{
		if (Registered.Component && Registered.Component->GetComponentLevel() == CurrentLevel)
		{
			CombinedHash = HashCombine(CombinedHash, Registered.ShapeHash);
		}
	}

	return CombinedHash;
#else
	return 0;
#endif
}

void FZoneGraphBuilder::BuildAll(const TArray<AZoneGraphData*>& AllZoneGraphData, const bool bForceRebuild)
{
	BuildData.Reset();

	bSkipHashCheck = bSkipHashCheck || bForceRebuild;

	bool bAllHashesMatch = true;
	TArray<uint32> PerDataShapeHash;
	PerDataShapeHash.SetNumZeroed(AllZoneGraphData.Num());
	for (int32 Index = 0; Index < AllZoneGraphData.Num(); Index++)
	{
		const AZoneGraphData* ZoneGraphData = AllZoneGraphData[Index];
		if (ZoneGraphData)
		{
			const uint32 CombinedHash = CalculateCombinedShapeHash(*ZoneGraphData);
			if (ZoneGraphData->GetCombinedShapeHash() != CombinedHash)
			{
				bAllHashesMatch = false;
			}
			PerDataShapeHash[Index] = CombinedHash;
		}
	}
	if (!bSkipHashCheck && bAllHashesMatch)
	{
		bIsDirty = false;
		return;
	}

	// Make sure all shape hashes are up to data.
	// We may have ended up here because some ZoneShape data has been update by code, and the hashes does not match.
	// Make sure the hashes match with the data we generate.
	// Note: The combined ZoneGraphData hash will be updated in Build(), not updating PerDataShapeHash[] here
	// since the invalidation check above should match the build loop below.
#if WITH_EDITOR
	for (FZoneGraphBuilderRegisteredComponent& Registered : ShapeComponents)
	{
		if (Registered.Component)
		{
			Registered.ShapeHash = Registered.Component->GetShapeHash();
		}
	}
#endif // WITH_EDITOR

	// Make sure all shape connections are up to date.
	for (FZoneGraphBuilderRegisteredComponent& Registered : ShapeComponents)
	{
		if (Registered.Component)
		{
			Registered.Component->UpdateConnectedShapes();
		}
	}

	// Build
	// TODO: build incrementally only affected data
	for (int32 Index = 0; Index < AllZoneGraphData.Num(); Index++)
	{
		AZoneGraphData* ZoneGraphData = AllZoneGraphData[Index];
		if (ZoneGraphData)
		{
			if (bSkipHashCheck || (ZoneGraphData->GetCombinedShapeHash() != PerDataShapeHash[Index]))
			{
				Build(*ZoneGraphData);
			}
		}
	}

	// We potentially changed the state of the component, make sure we'll update the visuals.
	for (FZoneGraphBuilderRegisteredComponent& Registered : ShapeComponents)
	{
		if (Registered.Component)
		{
			Registered.Component->MarkRenderStateDirty();
		}
	}

#if WITH_EDITOR
	UE::ZoneGraphDelegates::OnZoneGraphDataBuildDone.Broadcast(BuildData);
#endif

	bSkipHashCheck = false;
	bIsDirty = false;
}

void FZoneGraphBuilder::FindShapeConnections(const UZoneShapeComponent& SourceShapeComp, TArray<FZoneShapeConnection>& OutShapeConnections) const
{
	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
	check(ZoneGraphSettings);
	const FZoneGraphBuildSettings& BuildSettings = ZoneGraphSettings->GetBuildSettings();

	static const float ConnectionSnapDistance = BuildSettings.ConnectionSnapDistance;
	static const float ConnectionSnapAngleCos = FMath::Cos(FMath::DegreesToRadians(BuildSettings.ConnectionSnapAngle));

	const FTransform& SourceTransform = SourceShapeComp.GetComponentTransform();
	TConstArrayView<FZoneShapeConnector> SourceConnectors = SourceShapeComp.GetShapeConnectors();

	const ULevel* SourceLevel = SourceShapeComp.GetComponentLevel();

	OutShapeConnections.SetNum(SourceConnectors.Num());

	TArray<uint32> QueryResults;

	for (int32 i = 0; i < SourceConnectors.Num(); i++)
	{
		const FZoneShapeConnector& SourceConnector = SourceConnectors[i];
		const FVector SourceWorldPosition = SourceTransform.TransformPosition(SourceConnector.Position);
		const FVector SourceWorldNormal = SourceTransform.TransformVector(SourceConnector.Normal);

		// Reset connection
		OutShapeConnections[i].ShapeComponent = nullptr;
		OutShapeConnections[i].ConnectorIndex = 0;

		bool bFound = false;

		QueryResults.Reset();
		HashGrid.Query(FBox::BuildAABB(SourceWorldPosition, FVector(ConnectionSnapDistance)), QueryResults);

		for (uint32 Index : QueryResults)
		{
			check(ShapeComponents.IsValidIndex(int32(Index)));
			UZoneShapeComponent* DestShapeComp = ShapeComponents[Index].Component;
			if (!DestShapeComp)
			{
				continue;
			}
			if (DestShapeComp == &SourceShapeComp)
			{
				// Skip self
				continue;
			}
			if (SourceLevel != DestShapeComp->GetComponentLevel())
			{
				// Skip different worlds.
				continue;
			}

			const FTransform& DestTransform = DestShapeComp->GetComponentTransform();
			TConstArrayView<FZoneShapeConnector> DestConnectors = DestShapeComp->GetShapeConnectors();

			for (int32 j = 0; j < DestConnectors.Num(); j++)
			{
				const FZoneShapeConnector& DestConnector = DestConnectors[j];
				const FVector DestWorldPosition = DestTransform.TransformPosition(DestConnector.Position);
				const FVector DestWorldNormal = DestTransform.TransformVector(DestConnector.Normal);

				if (SourceConnector.LaneProfile == DestConnector.LaneProfile
					&& FVector::Dist(SourceWorldPosition, DestWorldPosition) < ConnectionSnapDistance
					&& FVector::DotProduct(SourceWorldNormal, -DestWorldNormal) > ConnectionSnapAngleCos)
				{
					// Check that the profile orientation matches before connecting.
					if (const FZoneLaneProfile* LaneProfile = ZoneGraphSettings->GetLaneProfileByRef(SourceConnector.LaneProfile))
					{
						if (LaneProfile->IsSymmetrical() || SourceConnector.bReverseLaneProfile != DestConnector.bReverseLaneProfile)
						{
							OutShapeConnections[i].ShapeComponent = DestShapeComp;
							OutShapeConnections[i].ConnectorIndex = j;
							bFound = true;
							break;
						}
					}
				}
			}

			if (bFound)
			{
				break;
			}
		}
	}
}

void FZoneGraphBuilder::BuildSingleShape(const UZoneShapeComponent& ShapeComp, const FMatrix& LocalToWorld, FZoneGraphStorage& OutZoneStorage)
{
	TArray<FZoneShapeLaneInternalLink> InternalLinks;
	AppendShapeToZoneStorage(ShapeComp, LocalToWorld, OutZoneStorage, InternalLinks);
	ConnectLanes(InternalLinks, OutZoneStorage);
}

void FZoneGraphBuilder::AppendShapeToZoneStorage(const UZoneShapeComponent& ShapeComp, const FMatrix& LocalToWorld,
												 FZoneGraphStorage& OutZoneStorage, TArray<FZoneShapeLaneInternalLink>& OutInternalLinks,
												 FZoneGraphBuildData* InBuildData)
{
	// Adjust polygon points based on connections.
	TArray<FZoneShapePoint> AdjustedPoints(ShapeComp.GetPoints());
	TConstArrayView<FZoneShapeConnector> ShapeConnectors = ShapeComp.GetShapeConnectors();
	TConstArrayView<FZoneShapeConnection> ConnectedShapes = ShapeComp.GetConnectedShapes();

	if (ConnectedShapes.Num() == ShapeConnectors.Num())
	{
		FTransform WorldToSource = ShapeComp.GetComponentTransform().Inverse();

		// The points that are connectors will be moved so that they align perfectly with the connect shape's geometry.
		for (int32 i = 0; i < ShapeConnectors.Num(); i++)
		{
			const FZoneShapeConnector& SourceConnector = ShapeConnectors[i];
			const FZoneShapeType SourceShapeType = ShapeComp.GetShapeType();

			const FZoneShapeConnection& Connection = ConnectedShapes[i];
			if (const UZoneShapeComponent* DestShapeComp = Connection.ShapeComponent.Get())
			{
				TConstArrayView<FZoneShapeConnector> DestConnectors = DestShapeComp->GetShapeConnectors();
				check(Connection.ConnectorIndex < DestConnectors.Num());
				const FZoneShapeConnector& DestConnector = DestConnectors[Connection.ConnectorIndex];
				const FZoneShapeType DestShapeType = DestShapeComp->GetShapeType();

				// When connecting spline to polygon, only adjust the polygon, otherwise adjust both shapes equally.
				float BlendFactor = 1.0f;
				if (SourceShapeType == FZoneShapeType::Spline)
				{
					if (DestShapeType == FZoneShapeType::Spline)
					{
						BlendFactor = 0.5f;
					}
					else // Polygon
					{
						BlendFactor = 0.0f;
					}
				}
				else // Polygon
				{
					if (DestShapeType == FZoneShapeType::Spline)
					{
						BlendFactor = 1.0f;
					}
					else // Polygon
					{
						BlendFactor = 0.5f;
					}
				}

				if (BlendFactor > 0.0f)
				{
					// Convert dest position and normal to source space
					const FTransform DestToSource = DestShapeComp->GetComponentTransform() * WorldToSource;
					const FVector LocalDestPosition = DestToSource.TransformPosition(DestConnector.Position);
					const FVector LocalDestNormal = DestToSource.TransformVector(DestConnector.Normal);
					const FVector LocalDestUp = DestToSource.TransformVector(DestConnector.Up);

					const FVector NewPosition = FMath::Lerp(SourceConnector.Position, LocalDestPosition, BlendFactor);
					const FVector NewNormal = FMath::Lerp(SourceConnector.Normal, -LocalDestNormal, BlendFactor).GetSafeNormal();
					const FVector NewUp = FMath::Lerp(SourceConnector.Up, LocalDestUp, BlendFactor).GetSafeNormal();

					if (SourceShapeType == FZoneShapeType::Spline)
					{
						// Adjust spline extremity.
						FZoneShapePoint& Point = AdjustedPoints[SourceConnector.PointIndex];
						Point.Position = NewPosition;
						// Connector and spline end points the same direction as spline, as connectors point out, spline start needs reversing.
						const float FlipNormal = SourceConnector.PointIndex == 0 ? -1.0f : 1.0f;
						Point.SetRotationFromForwardAndUp(NewNormal * FlipNormal, NewUp);
					}
					else
					{
						// Adjust polygon lane segment.
						FZoneShapePoint& Point = AdjustedPoints[SourceConnector.PointIndex];
						check(Point.Type == FZoneShapePointType::LaneProfile);
						Point.Position = NewPosition;
						// Connector normals point away from the shape, lane profile points rotations point in, need to reverse.
						Point.SetRotationFromForwardAndUp(-NewNormal, NewUp);
					}
				}
			}
		}
	}

	int32 ZoneIndex = OutZoneStorage.Zones.Num();
	if (ShapeComp.GetShapeType() == FZoneShapeType::Spline)
	{
		FZoneLaneProfile SplineLaneProfile;
		ShapeComp.GetSplineLaneProfile(SplineLaneProfile);
		if (ShapeComp.IsLaneProfileReversed())
		{
			SplineLaneProfile.ReverseLanes();
		}
		
		UE::ZoneShape::Utilities::TessellateSplineShape(AdjustedPoints, SplineLaneProfile, ShapeComp.GetTags(), LocalToWorld, OutZoneStorage, OutInternalLinks);
	}
	else if (ShapeComp.GetShapeType() == FZoneShapeType::Polygon)
	{
		TArray<FZoneLaneProfile> PolyLaneProfiles;
		ShapeComp.GetPolygonLaneProfiles(PolyLaneProfiles);

		TConstArrayView<FZoneShapePoint> Points = ShapeComp.GetPoints();
		check(Points.Num() == PolyLaneProfiles.Num());

		for (int32 i = 0; i < Points.Num(); i++)
		{
			const FZoneShapePoint& Point = Points[i];
			bool bReverse = Point.bReverseLaneProfile;
			if (Point.LaneProfile == FZoneShapePoint::InheritLaneProfile && ShapeComp.IsLaneProfileReversed())
			{
				bReverse = !bReverse;
			}
			if (bReverse)
			{
				PolyLaneProfiles[i].ReverseLanes();
			}
		}

		UE::ZoneShape::Utilities::TessellatePolygonShape(AdjustedPoints, ShapeComp.GetPolygonRoutingType(), PolyLaneProfiles, ShapeComp.GetTags(), LocalToWorld, OutZoneStorage, OutInternalLinks);
	}
	else
	{
		ensureMsgf(false, TEXT("Missing tessellation for shape type %d"), (int32)(ShapeComp.GetShapeType()));
		ZoneIndex = INDEX_NONE;
	}

	// Build mapping data between ZoneShapeComponents and baked data.
	if (InBuildData && ZoneIndex != INDEX_NONE)
	{
		FZoneData& Zone = OutZoneStorage.Zones[ZoneIndex];

		FZoneShapeComponentBuildData ComponentBuildData;
		ComponentBuildData.ZoneIndex = ZoneIndex;
		for (int32 i = Zone.LanesBegin; i != Zone.LanesEnd; i++)
		{
			ComponentBuildData.Lanes.Add(FZoneGraphLaneHandle(i, OutZoneStorage.DataHandle));
		}

		InBuildData->ZoneShapeComponentBuildData.Add(&ShapeComp, ComponentBuildData);
	}
}

void FZoneGraphBuilder::Build(AZoneGraphData& ZoneGraphData)
{
	FScopeLock RegistrationLock(&(ZoneGraphData.GetStorageLock()));

	ULevel* CurrentLevel = ZoneGraphData.GetLevel();
	ZoneGraphData.Modify();

	FZoneGraphStorage& ZoneStorage = ZoneGraphData.GetStorageMutable();
	ZoneStorage.Reset();

	TArray<FZoneShapeLaneInternalLink> InternalLinks;

	// Tessellate Zones
	for (FZoneGraphBuilderRegisteredComponent& Registered : ShapeComponents)
	{
		if (Registered.Component && Registered.Component->GetComponentLevel() == CurrentLevel)
		{
			// This feels backwards, maybe this could be part of the builder (i.e. static method) and the component would call it when needed. 
			AppendShapeToZoneStorage(*Registered.Component, Registered.Component->GetComponentTransform().ToMatrixWithScale(), ZoneStorage, InternalLinks, &BuildData);
		}
	}

	// Connect Lanes
	ConnectLanes(InternalLinks, ZoneStorage);

	// Update bounds for the whole data
	ZoneStorage.Bounds = FBox(ForceInit);
	for (FZoneData& Zone : ZoneStorage.Zones)
	{
		ZoneStorage.Bounds += Zone.Bounds;
	}

	// Build BV-tree for faster zone lookup.
	ZoneStorage.ZoneBVTree.Build(MakeStridedView(ZoneStorage.Zones, &FZoneData::Bounds));

	// Update combined hash
	const uint32 NewHash = CalculateCombinedShapeHash(ZoneGraphData);
	ZoneGraphData.SetCombinedShapeHash(NewHash);

	ZoneGraphData.UpdateDrawing();
}

void FZoneGraphBuilder::ConnectLanes(TArray<FZoneShapeLaneInternalLink>& InternalLinks, FZoneGraphStorage& ZoneStorage)
{
	enum class ELaneExtremity : uint8
	{
		Start,
		End,
	};

	struct FLanePointID
	{
		FLanePointID() = default;
		FLanePointID(const int32 InIndex, const ELaneExtremity InExtremity) : Index(InIndex), Extremity(InExtremity) {}

		int32 Index;
		ELaneExtremity Extremity;
	};

	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
	check(ZoneGraphSettings);
	const FZoneGraphBuildSettings& BuildSettings = ZoneGraphSettings->GetBuildSettings();

	// Build lookup for internal links
	TMap<int32, int32> FirstLinkByLane;

	// Make sure the links are in contiguous chunks.
	InternalLinks.Sort();

	// Make lookup to quickly find the first index based lane index.
	int32 PrevLaneIndex = INDEX_NONE;
	for (int32 LinkIdx = 0; LinkIdx < InternalLinks.Num(); LinkIdx++)
	{
		const FZoneShapeLaneInternalLink& Link = InternalLinks[LinkIdx];
		if (Link.LaneIndex != PrevLaneIndex)
		{
			FirstLinkByLane.Add(Link.LaneIndex, LinkIdx);
			PrevLaneIndex = Link.LaneIndex;
		}
	}

	// Add points to grid
	THierarchicalHashGrid2D<1, 1, FLanePointID> LinkGrid(100.0f);	// 1m grid size, no hierarchy, TODO: make configurable?

	static const float ConnectionTolerance = 2.0f; // Lanes that can connect should be really close to each other already because of the alignment snapping.
	static const float ConnectionToleranceSqr = FMath::Square(ConnectionTolerance);
	static const FVector ConnectionToleranceExtent(ConnectionTolerance);

	for (int32 LaneIdx = 0; LaneIdx < ZoneStorage.Lanes.Num(); LaneIdx++)
	{
		FZoneLaneData& Lane = ZoneStorage.Lanes[LaneIdx];
		LinkGrid.Add(FLanePointID(LaneIdx, ELaneExtremity::Start), FBox::BuildAABB(ZoneStorage.LanePoints[Lane.PointsBegin], FVector::ZeroVector));
		LinkGrid.Add(FLanePointID(LaneIdx, ELaneExtremity::End), FBox::BuildAABB(ZoneStorage.LanePoints[Lane.PointsEnd - 1], FVector::ZeroVector));
	}

	// Build lane connections
	TArray<FLanePointID> QueryResults;
	for (int32 LaneIndex = 0; LaneIndex < ZoneStorage.Lanes.Num(); LaneIndex++)
	{
		FZoneLaneData& Lane = ZoneStorage.Lanes[LaneIndex];
		Lane.LinksBegin = ZoneStorage.LaneLinks.Num();

		// Add internal links
		int32 AdjacentLaneCount = 0;
		if (const int32* FirstLink = FirstLinkByLane.Find(LaneIndex))
		{
			for (int32 LinkIdx = *FirstLink; LinkIdx < InternalLinks.Num(); LinkIdx++)
			{
				const FZoneShapeLaneInternalLink& Link = InternalLinks[LinkIdx];
				if (Link.LaneIndex != LaneIndex)
				{
					break;
				}
				ZoneStorage.LaneLinks.Add(Link.LinkData);
				if (Link.LinkData.Type == EZoneLaneLinkType::Adjacent)
				{
					AdjacentLaneCount++;
				}
			}
		}

		// Add links to connected lanes
		const FZoneLaneData& SourceLane = ZoneStorage.Lanes[LaneIndex];
		const FVector& SourceStartPosition = ZoneStorage.LanePoints[SourceLane.PointsBegin];
		const FVector& SourceEndPosition = ZoneStorage.LanePoints[SourceLane.PointsEnd - 1];

		// Lanes touching the source lane start point.
		QueryResults.Reset();
		LinkGrid.Query(FBox::BuildAABB(SourceStartPosition, ConnectionToleranceExtent), QueryResults);
		for (FLanePointID LaneID : QueryResults)
		{
			// skip self.
			if (LaneID.Index == LaneIndex)
			{
				continue;
			}

			const FZoneLaneData& DestLane = ZoneStorage.Lanes[LaneID.Index];
			const FVector& DestStartPosition = ZoneStorage.LanePoints[DestLane.PointsBegin];
			const FVector& DestEndPosition = ZoneStorage.LanePoints[DestLane.PointsEnd - 1];

			if (SourceLane.Tags.ContainsAny(DestLane.Tags & BuildSettings.LaneConnectionMask))
			{
				if (SourceLane.ZoneIndex != DestLane.ZoneIndex
					&& LaneID.Extremity == ELaneExtremity::End
					&& FVector::DistSquared(SourceStartPosition, DestEndPosition) < ConnectionToleranceSqr)
				{
					// Incoming lane
					FZoneLaneLinkData& Link = ZoneStorage.LaneLinks.AddDefaulted_GetRef();
					Link.DestLaneIndex = LaneID.Index;
					Link.Type = EZoneLaneLinkType::Incoming;
					Link.SetFlags(EZoneLaneLinkFlags::None);
				}
				else if (SourceLane.ZoneIndex == DestLane.ZoneIndex
						 && LaneID.Extremity == ELaneExtremity::Start
						 && FVector::DistSquared(SourceStartPosition, DestStartPosition) < ConnectionToleranceSqr)
				{
					// Splitting lane
					FZoneLaneLinkData& Link = ZoneStorage.LaneLinks.AddDefaulted_GetRef();
					Link.DestLaneIndex = LaneID.Index;
					Link.Type = EZoneLaneLinkType::Adjacent;
					Link.SetFlags(EZoneLaneLinkFlags::Splitting);
				}
			}
		}

		// Lanes touching the source lane end point.
		QueryResults.Reset();
		LinkGrid.Query(FBox::BuildAABB(SourceEndPosition, ConnectionToleranceExtent), QueryResults);
		for (FLanePointID LaneID : QueryResults)
		{
			// skip self.
			if (LaneID.Index == LaneIndex)
			{
				continue;
			}

			const FZoneLaneData& DestLane = ZoneStorage.Lanes[LaneID.Index];
			const FVector& DestStartPosition = ZoneStorage.LanePoints[DestLane.PointsBegin];
			const FVector& DestEndPosition = ZoneStorage.LanePoints[DestLane.PointsEnd - 1];

			if (SourceLane.Tags.ContainsAny(DestLane.Tags & BuildSettings.LaneConnectionMask))
			{
				if (SourceLane.ZoneIndex != DestLane.ZoneIndex
					&& LaneID.Extremity == ELaneExtremity::Start
					&& FVector::DistSquared(SourceEndPosition, DestStartPosition) < ConnectionToleranceSqr)
				{
					// Outgoing lane
					FZoneLaneLinkData& Link = ZoneStorage.LaneLinks.AddDefaulted_GetRef();
					Link.DestLaneIndex = LaneID.Index;
					Link.Type = EZoneLaneLinkType::Outgoing;
					Link.SetFlags(EZoneLaneLinkFlags::None);
				}
				else if (SourceLane.ZoneIndex == DestLane.ZoneIndex
						 && LaneID.Extremity == ELaneExtremity::End
						 && FVector::DistSquared(SourceEndPosition, DestEndPosition) < ConnectionToleranceSqr)
				{
					// Merging lane
					FZoneLaneLinkData& Link = ZoneStorage.LaneLinks.AddDefaulted_GetRef();
					Link.DestLaneIndex = LaneID.Index;
					Link.Type = EZoneLaneLinkType::Adjacent;
					Link.SetFlags(EZoneLaneLinkFlags::Merging);
				}
			}
		}

		// Potentially adjacent lanes in a polygons shape.
		if (AdjacentLaneCount == 0)
		{
			const float AdjacentRadius = SourceLane.Width + ConnectionTolerance; // Assumes adjacent lanes have same width.
			const float AdjacentRadiusSqr = FMath::Square(AdjacentRadius);
			const FVector AdjacentExt(AdjacentRadius);
			QueryResults.Reset();
			LinkGrid.Query(FBox::BuildAABB(SourceStartPosition, AdjacentExt), QueryResults);

			const FVector SourceStartSide = FVector::CrossProduct(ZoneStorage.LaneTangentVectors[SourceLane.PointsBegin], ZoneStorage.LaneUpVectors[SourceLane.PointsBegin]);
			const FVector SourceEndSide = FVector::CrossProduct(ZoneStorage.LaneTangentVectors[SourceLane.PointsEnd - 1], ZoneStorage.LaneUpVectors[SourceLane.PointsEnd - 1]);
			
			for (FLanePointID LaneID : QueryResults)
			{
				// skip self.
				if (LaneID.Index == LaneIndex)
				{
					continue;
				}

				const FZoneLaneData& DestLane = ZoneStorage.Lanes[LaneID.Index];
				if (SourceLane.ZoneIndex == DestLane.ZoneIndex
					&& SourceLane.Tags.ContainsAny(DestLane.Tags & BuildSettings.LaneConnectionMask))
				{
					// If the link already exists, do not create a duplicate one.
					bool bLinkExists = false;
					for (int32 LinkIndex = Lane.LinksBegin; LinkIndex < ZoneStorage.LaneLinks.Num(); LinkIndex++)
					{
						const FZoneLaneLinkData& Link = ZoneStorage.LaneLinks[LinkIndex];
						if (Link.DestLaneIndex == LaneID.Index)
						{
							bLinkExists = true;
							break;
						}
					}
					if (bLinkExists)
					{
						continue;
					}

					const FVector& DestStartPosition = ZoneStorage.LanePoints[DestLane.PointsBegin];
					const FVector& DestEndPosition = ZoneStorage.LanePoints[DestLane.PointsEnd - 1];

					// Using range checks, since we assume that the points should not be overlapping.
					
					if (UE::ZoneGraph::Internal::InRange(FVector::DistSquared(SourceStartPosition, DestStartPosition), ConnectionToleranceSqr, AdjacentRadiusSqr)
						&& UE::ZoneGraph::Internal::InRange(FVector::DistSquared(SourceEndPosition, DestEndPosition), ConnectionToleranceSqr, AdjacentRadiusSqr))
					{
						// Same direction adjacent lanes
						const bool bStartIsLeft = FVector::DotProduct(SourceStartSide, DestStartPosition - SourceStartPosition) > 0.0f;
						const bool bEndIsLeft = FVector::DotProduct(SourceEndSide, DestEndPosition - SourceEndPosition) > 0.0f;
						
						// Expect the adjacent lane points to be same side of the lane at start and end.
						if (bStartIsLeft == bEndIsLeft)
						{
							FZoneLaneLinkData& Link = ZoneStorage.LaneLinks.AddDefaulted_GetRef();
							Link.DestLaneIndex = LaneID.Index;
							Link.Type = EZoneLaneLinkType::Adjacent;
							Link.SetFlags(bStartIsLeft ? EZoneLaneLinkFlags::Left : EZoneLaneLinkFlags::Right);
						}
					}
					else if (UE::ZoneGraph::Internal::InRange(FVector::DistSquared(SourceStartPosition, DestEndPosition), ConnectionToleranceSqr, AdjacentRadiusSqr)
						&& UE::ZoneGraph::Internal::InRange(FVector::DistSquared(SourceEndPosition, DestStartPosition), ConnectionToleranceSqr, AdjacentRadiusSqr))
					{
						// Opposite direction adjacent lanes
						const bool bStartIsLeft = FVector::DotProduct(SourceStartSide, DestEndPosition - SourceStartPosition) > 0.0f;
						const bool bEndIsLeft = FVector::DotProduct(SourceEndSide, DestStartPosition - SourceEndPosition) > 0.0f;

						// Expect the adjacent lane points to be same side of the lane at start and end.
						if (bStartIsLeft == bEndIsLeft)
						{
							FZoneLaneLinkData& Link = ZoneStorage.LaneLinks.AddDefaulted_GetRef();
							Link.DestLaneIndex = LaneID.Index;
							Link.Type = EZoneLaneLinkType::Adjacent;
							Link.SetFlags((bStartIsLeft ? EZoneLaneLinkFlags::Left : EZoneLaneLinkFlags::Right) | EZoneLaneLinkFlags::OppositeDirection);
						}
					}
				}		
			}
		}

		Lane.LinksEnd = ZoneStorage.LaneLinks.Num();
	}
}

void FZoneGraphBuilder::QueryHashGrid(const FBox& Bounds, TArray<FZoneGraphBuilderHashGrid2D::ItemIDType>& OutResults)
{
	HashGrid.Query(Bounds, OutResults);
}
