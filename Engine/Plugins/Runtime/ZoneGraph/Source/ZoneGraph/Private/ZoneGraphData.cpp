// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZoneGraphData.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "ZoneGraphTypes.h"
#include "ZoneGraphRenderingComponent.h"
#include "ZoneGraphSubsystem.h"
#include "ProfilingDebugging/ScopedTimers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ZoneGraphData)


AZoneGraphData::AZoneGraphData(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bEnableDrawing(true)
	, RenderingComp(nullptr)
{
	PrimaryActorTick.bCanEverTick = false;
	bNetLoadOnClient = true;
	SetCanBeDamaged(false);

#if !UE_BUILD_SHIPPING
	RenderingComp = CreateDefaultSubobject<UZoneGraphRenderingComponent>(TEXT("ZoneGraphRenderingComp"), true);
	RenderingComp->SetVisibility(bEnableDrawing);
	RootComponent = RenderingComp;
#endif

#if WITH_EDITORONLY_DATA
	bIsSpatiallyLoaded = false;
#endif
}


/*
Registering logic and test cases:

	Load in editor:
		- PostLoad() (subsystem not ready)
		- UZoneGraphSubsystem::Initialize() (registers all missing zones)

	Sub Level, load in editor:
		- PostLoad()
		- PreRegisterAllComponents()

	Show Level (editor, levels window)
		- PreRegisterAllComponents()

	Hide Level (editor, levels window)
		- PostUnregisterAllComponents()

	Create New ZoneGraphData (happens via UZoneGraphSubsystem)
		- PostActorCreated()

	Delete ZoneGraphData (editor)
		- Destroyed()

	Undo Deleted ZoneGraphData (editor)
		- PostEditUndo()

	PIE start
		- PostLoad() (subsystem not ready)
		- UZoneGraphSubsystem::Initialize() (registers all missing zones)

	PIE, load level
		- PostLoad()
		- PreRegisterAllComponents()

	PIE stop (loaded level)
		- PostUnregisterAllComponents()
		- EndPlay()
		- PostUnregisterAllComponents()

	PIE stop
		- EndPlay()
*/

void AZoneGraphData::PostLoad()
{
	// Handle Level load, PIE, SIE, game load, streaming.
	Super::PostLoad();

	// Temporary fix for newly added up vectors.
	// TODO: Remove this.
	if (ZoneStorage.LaneUpVectors.Num() != ZoneStorage.LanePoints.Num())
	{
		UE_LOGF(LogZoneGraph, Warning, "\'%ls\' Is missing LaneUpVectors, generating default up vectors.", *GetName());
		ZoneStorage.LaneUpVectors.Init(FVector::UpVector, ZoneStorage.LanePoints.Num());
	}
	if (ZoneStorage.LaneTangentVectors.Num() != ZoneStorage.LanePoints.Num())
	{
		UE_LOGF(LogZoneGraph, Warning, "\'%ls\' Is missing LaneTangentVectors, generating zero tangents.", *GetName());
		ZoneStorage.LaneTangentVectors.Init(FVector::ZeroVector, ZoneStorage.LanePoints.Num());
	}
	if (ZoneStorage.ZoneBVTree.GetNumNodes() == 0 && ZoneStorage.Zones.Num() > 0)
	{
		UE_LOGF(LogZoneGraph, Warning, "\'%ls\' Is missing ZoneBVTree, generating for %d zones...", *GetName(),ZoneStorage.Zones.Num());

		FAutoScopedDurationTimer Timer;
		Timer.Start();
		ZoneStorage.ZoneBVTree.Build(MakeStridedView(ZoneStorage.Zones, &FZoneData::Bounds));
		
		UE_LOGF(LogZoneGraph, Warning, "  - Generated in %.2fms", Timer.GetTime() * 1000.0);
	}

	const bool bSucceeded = RegisterWithSubsystem();
	UE_LOGF(LogZoneGraph, Verbose, "\'%ls\' (0x%llx) PostLoad - %ls", *GetName(), UPTRINT(this), bSucceeded ? TEXT("Succeeded") : TEXT("Failed"));
}

void AZoneGraphData::Destroyed()
{
	// Handle editor delete.
	const bool bSucceeded = UnregisterWithSubsystem();
	Super::Destroyed();
	UE_LOGF(LogZoneGraph, Verbose, "\'%ls\' (0x%llx) Destroyed - %ls", *GetName(), UPTRINT(this), bSucceeded ? TEXT("Succeeded") : TEXT("Failed"));
}

void AZoneGraphData::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	// Handle Level unload, PIE end, SIE end, game end.
	const bool bSucceeded = UnregisterWithSubsystem();
	Super::EndPlay(EndPlayReason);
	UE_LOGF(LogZoneGraph, Verbose, "\'%ls\' (0x%llx) EndPlay - %ls", *GetName(), UPTRINT(this), bSucceeded ? TEXT("Succeeded") : TEXT("Failed"));
}

void AZoneGraphData::PostActorCreated()
{
	// Register after being initially spawned.
	Super::PostActorCreated();
	const bool bSucceeded = RegisterWithSubsystem();
	UE_LOGF(LogZoneGraph, Verbose, "\'%ls\' (0x%llx) PostActorCreated - %ls", *GetName(), UPTRINT(this), bSucceeded ? TEXT("Succeeded") : TEXT("Failed"));
}

void AZoneGraphData::PreRegisterAllComponents()
{
	Super::PreRegisterAllComponents();

	// Handle UWorld::AddToWorld(), i.e. turning on level visibility
	if (const ULevel* Level = GetLevel())
	{
		// This function gets called in editor all the time, we're only interested the case where level is being added to world.
		if (Level->bIsAssociatingLevel)
		{
			const bool bSucceeded = RegisterWithSubsystem();
			UE_LOGF(LogZoneGraph, Verbose, "\'%ls\' (0x%llx) PreRegisterAllComponents - %ls", *GetName(), UPTRINT(this), bSucceeded ? TEXT("Succeeded") : TEXT("Failed"));
		}
	}
}

void AZoneGraphData::PostUnregisterAllComponents()
{
	// Handle UWorld::RemoveFromWorld(), i.e. turning off level visibility
	if (const ULevel* Level = GetLevel())
	{
		// This function gets called in editor all the time, we're only interested the case where level is being removed from world.
		if (Level->bIsDisassociatingLevel)
		{
			const bool bSucceeded = UnregisterWithSubsystem();
			UE_LOGF(LogZoneGraph, Verbose, "\'%ls\' (0x%llx) PostUnregisterAllComponents - %ls", *GetName(), UPTRINT(this), bSucceeded ? TEXT("Succeeded") : TEXT("Failed"));
		}
	}

	Super::PostUnregisterAllComponents();
}

#if WITH_EDITOR
void AZoneGraphData::PostEditUndo()
{
	Super::PostEditUndo();

	if (IsPendingKillPending())
	{
		const bool bSucceeded = UnregisterWithSubsystem();
		UE_LOGF(LogZoneGraph, Verbose, "\'%ls\' (0x%llx) PostEditUndo/PendingKill - %ls", *GetName(), UPTRINT(this), bSucceeded ? TEXT("Succeeded") : TEXT("Failed"));
	}
	else
	{
		const bool bSucceeded = RegisterWithSubsystem();
		UE_LOGF(LogZoneGraph, Verbose, "\'%ls\' (0x%llx) PostEditUndo - %ls", *GetName(), UPTRINT(this), bSucceeded ? TEXT("Succeeded") : TEXT("Failed"));
	}
}

void AZoneGraphData::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		const FName PropName = PropertyChangedEvent.Property->GetFName();
		if (PropName == GET_MEMBER_NAME_CHECKED(AZoneGraphData, bEnableDrawing))
		{
			UpdateDrawing();
		}
	}
}
#endif

void AZoneGraphData::UpdateDrawing() const
{
#if !UE_BUILD_SHIPPING
	if (RenderingComp)
	{
		RenderingComp->SetVisibility(bEnableDrawing);
		if (bEnableDrawing)
		{
			RenderingComp->MarkRenderStateDirty();
		}
	}
#endif // UE_BUILD_SHIPPING
}


FBox AZoneGraphData::GetBounds() const
{
	return ZoneStorage.Bounds;
}

bool AZoneGraphData::RegisterWithSubsystem()
{
	if (!bRegistered && HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		if (UZoneGraphSubsystem* ZoneGraph = UWorld::GetSubsystem<UZoneGraphSubsystem>(GetWorld()))
		{
			ZoneGraph->RegisterZoneGraphData(*this);
			UpdateDrawing();
			return true;
		}
	}
	return false;
}

bool AZoneGraphData::UnregisterWithSubsystem()
{
	if (bRegistered)
	{
		if (UZoneGraphSubsystem* ZoneGraph = UWorld::GetSubsystem<UZoneGraphSubsystem>(GetWorld()))
		{
			ZoneGraph->UnregisterZoneGraphData(*this);
			ZoneStorage.DataHandle.Reset();
			UpdateDrawing();
			return true;
		}
	}
	return false;
}

void AZoneGraphData::OnRegistered(const FZoneGraphDataHandle DataHandle)
{
	ZoneStorage.DataHandle = DataHandle;
	bRegistered = true;
}

void AZoneGraphData::OnUnregistered()
{
	bRegistered = false;
}
