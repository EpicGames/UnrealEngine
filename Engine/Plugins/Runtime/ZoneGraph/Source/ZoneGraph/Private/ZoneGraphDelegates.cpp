// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZoneGraphDelegates.h"
#include "CoreMinimal.h"

namespace UE
{
namespace ZoneGraphDelegates
{

#if WITH_EDITOR
FOnZoneGraphDataBuildDone ZONEGRAPH_API OnZoneGraphDataBuildDone;
FOnZoneGraphTagsChanged ZONEGRAPH_API OnZoneGraphTagsChanged;
FOnZoneGraphLaneProfileChanged ZONEGRAPH_API OnZoneGraphLaneProfileChanged;
FOnZoneGraphBuildSettingsChanged ZONEGRAPH_API OnZoneGraphBuildSettingsChanged;
FOnZoneGraphRequestRebuild OnZoneGraphRequestRebuild;
#endif

FOnZoneGraphData ZONEGRAPH_API OnPostZoneGraphDataAdded;
FOnZoneGraphData ZONEGRAPH_API OnPreZoneGraphDataRemoved;

} // ZoneGraphDelegates
} // UE
