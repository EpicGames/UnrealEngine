// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZoneGraphQuery.h"
#include "ZoneGraphTypes.h"
#include "Containers/ArrayView.h"
#include "Algo/BinarySearch.h"

namespace UE::ZoneGraph::Query
{

float ClosestTimeOnSegment(const FVector& Point, const FVector& StartPoint, const FVector& EndPoint)
{
	const FVector Segment = EndPoint - StartPoint;
	const FVector VectToPoint = Point - StartPoint;

	// See if closest point is before StartPoint
	const FVector::FReal Dot1 = VectToPoint | Segment;
	if (Dot1 <= 0.0)
	{
		return 0.0f;
	}

	// See if closest point is beyond EndPoint
	const FVector::FReal Dot2 = Segment | Segment;
	if (Dot2 <= Dot1)
	{
		return 1.0f;
	}

	// Closest Point is within segment
	return FloatCastChecked<float>(Dot1 / Dot2, /* Precision */ 1./128.);
}

static bool EnsureLaneHandle(const FZoneGraphStorage& Storage, const FZoneGraphLaneHandle LaneHandle, const char* Function)
{
	if (LaneHandle.DataHandle != Storage.DataHandle || LaneHandle.Index >= Storage.Lanes.Num())
	{
		ensureMsgf(false, TEXT("%s: Bad lane handle: index = %d data = %d/%d), expected: index < %d data = %d/%d)"), ANSI_TO_TCHAR(Function),
			LaneHandle.Index, LaneHandle.DataHandle.Index, LaneHandle.DataHandle.Generation, Storage.Lanes.Num(), Storage.DataHandle.Index, Storage.DataHandle.Generation);
		return false;
	}
	return true;
}


bool GetLaneLength(const FZoneGraphStorage& Storage, const FZoneGraphLaneHandle LaneHandle, float& OutLength)
{
	if (!EnsureLaneHandle(Storage, LaneHandle, __FUNCTION__))
	{
		return false;
	}
	return GetLaneLength(Storage, LaneHandle.Index, OutLength);
}

bool GetLaneLength(const FZoneGraphStorage& Storage, const uint32 LaneIndex, float& OutLength)
{
	const FZoneLaneData& Lane = Storage.Lanes[LaneIndex];
	OutLength = Storage.LanePointProgressions[Lane.PointsEnd - 1];
	return true;
}

bool GetLaneWidth(const FZoneGraphStorage& Storage, const FZoneGraphLaneHandle LaneHandle, float& OutWidth)
{
	//@todo: making sure that what is not part of the condition gets strip out for the shipping build
	if (!EnsureLaneHandle(Storage, LaneHandle, __FUNCTION__))
	{
		return false;
	}
	return GetLaneWidth(Storage, LaneHandle.Index, OutWidth);
}

bool GetLaneWidth(const FZoneGraphStorage& Storage, const uint32 LaneIndex, float& OutWidth)
{
	//@todo: ensure for LaneIndex to make consistent with the accessors receiving handles
	const FZoneLaneData& Lane = Storage.Lanes[LaneIndex];
	OutWidth = Lane.Width;
	return true;
}

bool GetLaneTags(const FZoneGraphStorage& Storage, const FZoneGraphLaneHandle& LaneHandle, FZoneGraphTagMask& OutTags)
{
	if (!EnsureLaneHandle(Storage, LaneHandle, __FUNCTION__))
	{
		return false;
	}
	return GetLaneTags(Storage, LaneHandle.Index, OutTags);
}

bool GetLaneTags(const FZoneGraphStorage& Storage, const uint32 LaneIndex, FZoneGraphTagMask& OutTags)
{
	const FZoneLaneData& Lane = Storage.Lanes[LaneIndex];
	OutTags = Lane.Tags;
	return true;
}

static bool LinkMatches(const FZoneLaneLinkData& Link, const EZoneLaneLinkType Types, const EZoneLaneLinkFlags IncludeFlags, const EZoneLaneLinkFlags ExcludeFlags)
{
	if ((Link.Type & Types) != EZoneLaneLinkType::None)
	{
		// Check flags only adjacent links.
		if (Link.Type == EZoneLaneLinkType::Adjacent)
		{
			return Link.HasFlags(IncludeFlags) && !Link.HasFlags(ExcludeFlags);
		}
		return true;
	}
	return false;
}

bool GetLinkedLanes(const FZoneGraphStorage& Storage, const uint32 LaneIndex, const EZoneLaneLinkType Types, const EZoneLaneLinkFlags IncludeFlags, const EZoneLaneLinkFlags ExcludeFlags, TArray<FZoneGraphLinkedLane>& OutLinkedLanes)
{
	const FZoneLaneData& Lane = Storage.Lanes[LaneIndex];
	OutLinkedLanes.Reset();

	for (int32 i = Lane.LinksBegin; i < Lane.LinksEnd; i++)
	{
		const FZoneLaneLinkData& Link = Storage.LaneLinks[i];
		if (LinkMatches(Link, Types, IncludeFlags, ExcludeFlags))
		{
			const FZoneGraphLaneHandle DestLaneHandle(Link.DestLaneIndex, Storage.DataHandle);
			OutLinkedLanes.Emplace(DestLaneHandle, Link.Type, (EZoneLaneLinkFlags)Link.Flags);
		}
	}

	return true;
}

bool GetLinkedLanes(const FZoneGraphStorage& Storage, const FZoneGraphLaneHandle LaneHandle, const EZoneLaneLinkType Types, const EZoneLaneLinkFlags IncludeFlags, const EZoneLaneLinkFlags ExcludeFlags, TArray<FZoneGraphLinkedLane>& OutLinkedLanes)
{
	if (!EnsureLaneHandle(Storage, LaneHandle, __FUNCTION__))
	{
		return false;
	}
	return GetLinkedLanes(Storage, LaneHandle.Index, Types, IncludeFlags, ExcludeFlags, OutLinkedLanes);
}


bool GetFirstLinkedLane(const FZoneGraphStorage& Storage, const uint32 LaneIndex, const EZoneLaneLinkType Types, const EZoneLaneLinkFlags IncludeFlags, const EZoneLaneLinkFlags ExcludeFlags, FZoneGraphLinkedLane& OutLinkedLane)
{
	const FZoneLaneData& Lane = Storage.Lanes[LaneIndex];
	OutLinkedLane.Reset();

	for (int32 i = Lane.LinksBegin; i < Lane.LinksEnd; i++)
	{
		const FZoneLaneLinkData& Link = Storage.LaneLinks[i];
		if (LinkMatches(Link, Types, IncludeFlags, ExcludeFlags))
		{
			OutLinkedLane.DestLane = FZoneGraphLaneHandle(Link.DestLaneIndex, Storage.DataHandle);
			OutLinkedLane.Type = Link.Type;
			OutLinkedLane.Flags = Link.Flags;
			break;
		}
	}

	return true;
}

bool GetFirstLinkedLane(const FZoneGraphStorage& Storage, const FZoneGraphLaneHandle LaneHandle, const EZoneLaneLinkType Types, const EZoneLaneLinkFlags IncludeFlags, const EZoneLaneLinkFlags ExcludeFlags, FZoneGraphLinkedLane& OutLinkedLane)
{
	if (!EnsureLaneHandle(Storage, LaneHandle, __FUNCTION__))
	{
		return false;
	}
	return GetFirstLinkedLane(Storage, LaneHandle.Index, Types, IncludeFlags, ExcludeFlags, OutLinkedLane);
}

static bool AdvanceLaneLocationInternal(const FZoneGraphStorage& Storage, const FZoneGraphLaneLocation& InLaneLocation, const float AdvanceDistance, FZoneGraphLaneLocation& OutLaneLocation, TArray<FZoneGraphLaneLocation>* OutAllLaneLocations = nullptr)
{
	if (!EnsureLaneHandle(Storage, InLaneLocation.LaneHandle, __FUNCTION__))
	{
		return false;
	}

	const FZoneLaneData& Lane = Storage.Lanes[InLaneLocation.LaneHandle.Index];
	const int32 NumLanePoints = Lane.PointsEnd - Lane.PointsBegin;
	check(NumLanePoints >= 2);

	int32 CurrentSegment = InLaneLocation.LaneSegment;
	check(CurrentSegment < Storage.LanePoints.Num());

	// Advance along lane to find the location
	float AdvancedDistance = InLaneLocation.DistanceAlongLane + FMath::Max(0.0f, AdvanceDistance);

	float d = Storage.LanePointProgressions[CurrentSegment];

	// Advance to the segment containing the new distance.
	while (CurrentSegment < (Lane.PointsEnd - 2))
	{
		const float SegEndDistance = Storage.LanePointProgressions[CurrentSegment + 1];
		if (AdvancedDistance < SegEndDistance)
		{
			break;
		}
		CurrentSegment++;

		if (OutAllLaneLocations != nullptr)
		{
			FZoneGraphLaneLocation& LaneLocation = (*OutAllLaneLocations).AddDefaulted_GetRef();
			LaneLocation.LaneHandle = InLaneLocation.LaneHandle;
			LaneLocation.DistanceAlongLane = SegEndDistance;
			LaneLocation.LaneSegment = CurrentSegment;
			LaneLocation.Position = Storage.LanePoints[CurrentSegment];
			LaneLocation.Direction = (Storage.LanePoints[CurrentSegment+1] - Storage.LanePoints[CurrentSegment]).GetSafeNormal();
			LaneLocation.Tangent = Storage.LaneTangentVectors[CurrentSegment];
			LaneLocation.Up = Storage.LaneUpVectors[CurrentSegment];
		}
	}
	// Clamp advanced distance to lane length.
	AdvancedDistance = FMath::Min(AdvancedDistance, Storage.LanePointProgressions[CurrentSegment + 1]);

	// Interpolate location along the lane.
	const float SegLength = Storage.LanePointProgressions[CurrentSegment + 1] - Storage.LanePointProgressions[CurrentSegment];
	const float InvSegLength = SegLength > KINDA_SMALL_NUMBER ? 1.0f / SegLength : 0.0f;
	const float t = FMath::Clamp((AdvancedDistance - Storage.LanePointProgressions[CurrentSegment]) * InvSegLength, 0.0f, 1.0f);

	OutLaneLocation.LaneHandle = InLaneLocation.LaneHandle;
	OutLaneLocation.DistanceAlongLane = AdvancedDistance;
	OutLaneLocation.LaneSegment = CurrentSegment;
	OutLaneLocation.Position = FMath::Lerp(Storage.LanePoints[CurrentSegment], Storage.LanePoints[CurrentSegment + 1], t);
	OutLaneLocation.Direction = (Storage.LanePoints[CurrentSegment + 1] - Storage.LanePoints[CurrentSegment]).GetSafeNormal();
	OutLaneLocation.Tangent = FMath::Lerp(Storage.LaneTangentVectors[CurrentSegment], Storage.LaneTangentVectors[CurrentSegment + 1], t).GetSafeNormal();
	OutLaneLocation.Up = FMath::Lerp(Storage.LaneUpVectors[CurrentSegment], Storage.LaneUpVectors[CurrentSegment + 1], t).GetSafeNormal();

	if (OutAllLaneLocations != nullptr)
	{
		(*OutAllLaneLocations).Add(OutLaneLocation);
	}
	return true;
}
	
bool AdvanceLaneLocation(const FZoneGraphStorage& Storage, const FZoneGraphLaneLocation& InLaneLocation, const float AdvanceDistance, FZoneGraphLaneLocation& OutLaneLocation)
{
	return AdvanceLaneLocationInternal(Storage, InLaneLocation, AdvanceDistance, OutLaneLocation);
}

bool AdvanceLaneLocation(const FZoneGraphStorage& Storage, const FZoneGraphLaneLocation& InLaneLocation, const float AdvanceDistance, TArray<FZoneGraphLaneLocation>& OutLaneLocations)
{
	FZoneGraphLaneLocation DummyLocation;
	return AdvanceLaneLocationInternal(Storage, InLaneLocation, AdvanceDistance, DummyLocation, &OutLaneLocations);
}

bool CalculateLaneSegmentIndexAtDistance(const FZoneGraphStorage& Storage, const FZoneGraphLaneHandle LaneHandle, const float Distance, int32& OutSegmentIndex)
{
	if (!EnsureLaneHandle(Storage, LaneHandle, __FUNCTION__))
	{
		return false;
	}
	return CalculateLaneSegmentIndexAtDistance(Storage, LaneHandle.Index, Distance, OutSegmentIndex);
}

bool CalculateLaneSegmentIndexAtDistance(const FZoneGraphStorage& Storage, const uint32 LaneIndex, const float Distance, int32& OutSegmentIndex)
{
	const FZoneLaneData& Lane = Storage.Lanes[LaneIndex];
	const int32 NumLanePoints = Lane.PointsEnd - Lane.PointsBegin;
	check(NumLanePoints >= 2);

	// Two or more points from here on.
	if (Distance <= Storage.LanePointProgressions[Lane.PointsBegin])
	{
		// Handle out of range, before.
		OutSegmentIndex = Lane.PointsBegin;
	}
	else if (Distance >= Storage.LanePointProgressions[Lane.PointsEnd - 1])
	{
		// Handle out of range, after.
		OutSegmentIndex = Lane.PointsEnd - 2;
	}
	else
	{
		// Binary search correct segment.
		TArrayView<const float> LanePointProgressionsView(&Storage.LanePointProgressions[Lane.PointsBegin], NumLanePoints);
		// We want SegStart/End to be consecutive indices withing the range of the lane points.
		// Upper bound finds the point past the distance, so we sub 1, and then clamp it to fit valid range.
		OutSegmentIndex = FMath::Clamp(Algo::UpperBound(LanePointProgressionsView, Distance) - 1, 0, NumLanePoints - 2) + Lane.PointsBegin;
	}

	return true;
}

bool CalculateLocationAlongLaneFromRatio(const FZoneGraphStorage& Storage, const FZoneGraphLaneHandle LaneHandle, const float Ratio, FZoneGraphLaneLocation& OutLaneLocation)
{
	if (!EnsureLaneHandle(Storage, LaneHandle, __FUNCTION__))
	{
		return false;
	}
	return CalculateLocationAlongLaneFromRatio(Storage, LaneHandle.Index, Ratio, OutLaneLocation);
}

bool CalculateLocationAlongLaneFromRatio(const FZoneGraphStorage& Storage, const uint32 LaneIndex, const float Ratio, FZoneGraphLaneLocation& OutLaneLocation)
{
	const float ValidRatio = ensureMsgf(Ratio >= 0.0f && Ratio <= 1.0f, TEXT("Ratio must be in range [0..1]")) ? Ratio : FMath::Clamp(Ratio, 0.f, 1.0f);
	float LaneLength = 0.0f;
	if (!GetLaneLength(Storage, LaneIndex, LaneLength))
	{
		return false;
	}

	return CalculateLocationAlongLane(Storage, LaneIndex, LaneLength * ValidRatio, OutLaneLocation);
}

bool CalculateLocationAlongLane(const FZoneGraphStorage& Storage, const FZoneGraphLaneHandle LaneHandle, const float Distance, FZoneGraphLaneLocation& OutLaneLocation)
{
	if (!EnsureLaneHandle(Storage, LaneHandle, __FUNCTION__))
	{
		return false;
	}
	return CalculateLocationAlongLane(Storage, LaneHandle.Index, Distance, OutLaneLocation);
}

bool CalculateLocationAlongLane(const FZoneGraphStorage& Storage, const uint32 LaneIndex, const float Distance, FZoneGraphLaneLocation& OutLaneLocation)
{
	const FZoneLaneData& Lane = Storage.Lanes[LaneIndex];
	const int32 NumLanePoints = Lane.PointsEnd - Lane.PointsBegin;
	check(NumLanePoints >= 2);

	// Two or more points from here on.
	float t = 0.0f;
	float ClampedDistance = Distance;
	int32 SegStart = 0, SegEnd = 1;
	if (Distance <= Storage.LanePointProgressions[Lane.PointsBegin])
	{
		// Handle out of range, before.
		SegStart = Lane.PointsBegin;
		SegEnd = Lane.PointsBegin + 1;
		t = 0.0f;
		ClampedDistance = 0.0f;
	}
	else if (Distance >= Storage.LanePointProgressions[Lane.PointsEnd - 1])
	{
		// Handle out of range, after.
		SegStart = Lane.PointsEnd - 2;
		SegEnd = Lane.PointsEnd - 1;
		t = 1.0f;
		ClampedDistance = Storage.LanePointProgressions[Lane.PointsEnd - 1];
	}
	else
	{
		if (NumLanePoints == 2)
		{
			SegStart = Lane.PointsBegin;
			SegEnd = Lane.PointsBegin + 1;
		}
		else
		{
			// Binary search correct segment.
			TArrayView<const float> LanePointProgressionsView(&Storage.LanePointProgressions[Lane.PointsBegin], NumLanePoints);
			// We want SegStart/End to be consecutive indices withing the range of the lane points.
			// Upper bound finds the point past the distance, so we sub 1, and then clamp it to fit valid range.
			SegStart = FMath::Clamp(Algo::UpperBound(LanePointProgressionsView, Distance) - 1, 0, NumLanePoints - 2) + Lane.PointsBegin;
			SegEnd = SegStart + 1;
		}
		const float SegLength = Storage.LanePointProgressions[SegEnd] - Storage.LanePointProgressions[SegStart];
		const float InvSegLength = SegLength > KINDA_SMALL_NUMBER ? 1.0f / SegLength : 0.0f;
		t = FMath::Clamp((Distance - Storage.LanePointProgressions[SegStart]) * InvSegLength, 0.0f, 1.0f);
	}

	OutLaneLocation.LaneHandle = FZoneGraphLaneHandle(LaneIndex, Storage.DataHandle);
	OutLaneLocation.DistanceAlongLane = ClampedDistance;
	OutLaneLocation.LaneSegment = SegStart;
	OutLaneLocation.Position = FMath::Lerp(Storage.LanePoints[SegStart], Storage.LanePoints[SegEnd], t);
	OutLaneLocation.Direction = (Storage.LanePoints[SegEnd] - Storage.LanePoints[SegStart]).GetSafeNormal();
	OutLaneLocation.Tangent = FMath::Lerp(Storage.LaneTangentVectors[SegStart], Storage.LaneTangentVectors[SegEnd], t).GetSafeNormal();
	OutLaneLocation.Up = FMath::Lerp(Storage.LaneUpVectors[SegStart], Storage.LaneUpVectors[SegEnd], t).GetSafeNormal();

	return true;
}

static bool FindNearestLocationOnLaneInternal(const FZoneGraphStorage& Storage, const int32 LaneIndex, const FVector& Center, const float RangeSqr, FZoneGraphLaneLocation& OutLaneLocation, float& OutDistanceSqr)
{
	const FZoneLaneData& Lane = Storage.Lanes[LaneIndex];

	float NearestDistanceSqr = RangeSqr;
	int32 NearestLaneSegment = 0;
	float NearestLaneSegmentT = 0.0f;
	FVector NearestLanePosition(ForceInitToZero);
	bool bResult = false;

	for (int32 i = Lane.PointsBegin; i < Lane.PointsEnd - 1; i++)
	{
		const FVector& SegStart = Storage.LanePoints[i];
		const FVector& SegEnd = Storage.LanePoints[i + 1];
		const float SegT = ClosestTimeOnSegment(Center, SegStart, SegEnd);
		const FVector ClosestPt = FMath::Lerp(SegStart, SegEnd, SegT);
		const float DistSqr = FVector::DistSquared(Center, ClosestPt);
		if (DistSqr < NearestDistanceSqr)
		{
			NearestDistanceSqr = DistSqr;
			NearestLaneSegment = i;
			NearestLaneSegmentT = SegT;
			NearestLanePosition = ClosestPt;
			bResult = true;
		}
	}

	if (bResult)
	{
		OutLaneLocation.LaneHandle.DataHandle = Storage.DataHandle;
		OutLaneLocation.LaneHandle.Index = LaneIndex;
		OutLaneLocation.LaneSegment = NearestLaneSegment;
		OutLaneLocation.DistanceAlongLane = FMath::Lerp(Storage.LanePointProgressions[NearestLaneSegment], Storage.LanePointProgressions[NearestLaneSegment + 1], NearestLaneSegmentT);
		OutLaneLocation.Position = NearestLanePosition;
		OutLaneLocation.Direction = (Storage.LanePoints[NearestLaneSegment + 1] - Storage.LanePoints[NearestLaneSegment]).GetSafeNormal();
		OutLaneLocation.Tangent = FMath::Lerp(Storage.LaneTangentVectors[NearestLaneSegment], Storage.LaneTangentVectors[NearestLaneSegment + 1], NearestLaneSegmentT).GetSafeNormal();
		OutLaneLocation.Up = FMath::Lerp(Storage.LaneUpVectors[NearestLaneSegment], Storage.LaneUpVectors[NearestLaneSegment + 1], NearestLaneSegmentT).GetSafeNormal();
		OutDistanceSqr = NearestDistanceSqr;
	}
	else
	{
		OutLaneLocation.Reset();
		OutDistanceSqr = 0.0f;
	}

	return bResult;
}

	
bool FindNearestLocationOnLane(const FZoneGraphStorage& Storage, const FZoneGraphLaneHandle LaneHandle, const FBox& Bounds, FZoneGraphLaneLocation& OutLaneLocation, float& OutDistanceSqr)
{
	if (!EnsureLaneHandle(Storage, LaneHandle, __FUNCTION__))
	{
		return false;
	}

	if (!FindNearestLocationOnLaneInternal(Storage, LaneHandle.Index, Bounds.GetCenter(), Bounds.GetExtent().SizeSquared(), OutLaneLocation, OutDistanceSqr))
	{
		return false;
	}

	// Make sure the nearest point is inside the box
	if (!Bounds.IsInside(OutLaneLocation.Position))
	{
		OutLaneLocation.Reset();
		OutDistanceSqr = 0.0f;
		return false;
	}

	return true;
}

bool FindNearestLocationOnLane(const FZoneGraphStorage& Storage, const FZoneGraphLaneHandle LaneHandle, const FVector& Center, const float Range, FZoneGraphLaneLocation& OutLaneLocation, float& OutDistanceSqr)
{
	if (!EnsureLaneHandle(Storage, LaneHandle, __FUNCTION__))
	{
		return false;
	}

	return FindNearestLocationOnLaneInternal(Storage, LaneHandle.Index, Center, FMath::Square(Range), OutLaneLocation, OutDistanceSqr);
}
	
bool FindNearestLane(const FZoneGraphStorage& Storage, const FBox& Bounds, const FZoneGraphTagFilter TagFilter, FZoneGraphLaneLocation& OutLaneLocation, float& OutDistanceSqr)
{
	struct FQueryResult
	{
		float NearestDistanceSqr = 0.0f;
		int32 NearestLaneIdx = 0;
		int32 NearestLaneSegment = 0;
		float NearestLaneSegmentT = 0;
		FVector NearestLanePosition = FVector::ZeroVector;
		bool bValid = false;
	};
	
	FQueryResult Result;
	Result.NearestDistanceSqr = Bounds.GetExtent().SizeSquared();

	FVector Center(Bounds.GetCenter());

	Storage.ZoneBVTree.Query(Bounds, [&Storage, &Bounds, &TagFilter, &Center, &Result](const FZoneGraphBVNode& Node)
	{
		const FZoneData& Zone = Storage.Zones[Node.Index];
		for (int32 LaneIdx = Zone.LanesBegin; LaneIdx < Zone.LanesEnd; LaneIdx++)
		{
			const FZoneLaneData& Lane = Storage.Lanes[LaneIdx];
			if (TagFilter.Pass(Lane.Tags))
			{
				for (int32 i = Lane.PointsBegin; i < Lane.PointsEnd - 1; i++)
				{
					const FVector& SegStart = Storage.LanePoints[i];
					const FVector& SegEnd = Storage.LanePoints[i + 1];
					const float SegT = ClosestTimeOnSegment(Center, SegStart, SegEnd);
					const FVector ClosestPt = FMath::Lerp(SegStart, SegEnd, SegT);
					if (Bounds.IsInside(ClosestPt))
					{
						const float DistSqr = FVector::DistSquared(Center, ClosestPt);
						if (DistSqr < Result.NearestDistanceSqr)
						{
							Result.NearestDistanceSqr = DistSqr;
							Result.NearestLaneIdx = LaneIdx;
							Result.NearestLaneSegment = i;
							Result.NearestLaneSegmentT = SegT;
							Result.NearestLanePosition = ClosestPt;
							Result.bValid = true;
						}
					}
				}
			}
		}
	});

	if (Result.bValid)
	{
		OutLaneLocation.LaneHandle.DataHandle = Storage.DataHandle;
		OutLaneLocation.LaneHandle.Index = uint32(Result.NearestLaneIdx);
		OutLaneLocation.LaneSegment = Result.NearestLaneSegment;
		OutLaneLocation.DistanceAlongLane = FMath::Lerp(Storage.LanePointProgressions[Result.NearestLaneSegment], Storage.LanePointProgressions[Result.NearestLaneSegment + 1], Result.NearestLaneSegmentT);
		OutLaneLocation.Position = Result.NearestLanePosition;
		OutLaneLocation.Direction = (Storage.LanePoints[Result.NearestLaneSegment + 1] - Storage.LanePoints[Result.NearestLaneSegment]).GetSafeNormal();
		OutLaneLocation.Tangent = FMath::Lerp(Storage.LaneTangentVectors[Result.NearestLaneSegment], Storage.LaneTangentVectors[Result.NearestLaneSegment + 1], Result.NearestLaneSegmentT).GetSafeNormal();
		OutLaneLocation.Up = FMath::Lerp(Storage.LaneUpVectors[Result.NearestLaneSegment], Storage.LaneUpVectors[Result.NearestLaneSegment + 1], Result.NearestLaneSegmentT).GetSafeNormal();
		OutDistanceSqr = Result.NearestDistanceSqr;
	}
	else
	{
		OutLaneLocation.Reset();
		OutDistanceSqr = 0.0f;
	}

	return Result.bValid;
}

bool FindOverlappingLanes(const FZoneGraphStorage& Storage, const FBox& Bounds, const FZoneGraphTagFilter TagFilter, TArray<FZoneGraphLaneHandle>& OutLanes)
{
	Storage.ZoneBVTree.Query(Bounds, [&Storage, &Bounds, &OutLanes, &TagFilter](const FZoneGraphBVNode& Node)
	{
		const FZoneData& Zone = Storage.Zones[Node.Index];
		
		for (int32 LaneIdx = Zone.LanesBegin; LaneIdx < Zone.LanesEnd; LaneIdx++)
		{
			const FZoneLaneData& Lane = Storage.Lanes[LaneIdx];
			if (TagFilter.Pass(Lane.Tags))
			{
				for (int32 i = Lane.PointsBegin; i < Lane.PointsEnd - 1; i++)
				{
					const FVector& SegStart = Storage.LanePoints[i];
					const FVector& SegEnd = Storage.LanePoints[i + 1];
					if (FMath::LineBoxIntersection(Bounds, SegStart, SegEnd, SegEnd - SegStart))
					{
						OutLanes.Emplace(uint32(LaneIdx), Storage.DataHandle);
						break;
					}
				}
			}
		}
	});

	return OutLanes.Num() > 0;
}

bool FindLaneOverlaps(const FZoneGraphStorage& Storage, const FVector& Center, const float Radius, const FZoneGraphTagFilter TagFilter, TArray<FZoneGraphLaneSection>& OutLaneSections)
{
	FBox Bounds(Center, Center);
	Bounds = Bounds.ExpandBy(Radius);

	Storage.ZoneBVTree.Query(Bounds, [&Storage, Center, Radius, &OutLaneSections, &TagFilter](const FZoneGraphBVNode& Node)
	{
		const FZoneData& Zone = Storage.Zones[Node.Index];
		
		for (int32 LaneIdx = Zone.LanesBegin; LaneIdx < Zone.LanesEnd; LaneIdx++)
		{
			const FZoneLaneData& Lane = Storage.Lanes[LaneIdx];
			if (TagFilter.Pass(Lane.Tags))
			{
				for (int32 i = Lane.PointsBegin; i < Lane.PointsEnd - 1; i++)
				{
					const FVector& SegStart = Storage.LanePoints[i];
					const FVector& SegEnd = Storage.LanePoints[i + 1];

					const float SegT = ClosestTimeOnSegment(Center, SegStart, SegEnd);
					const FVector ClosestPt = FMath::Lerp(SegStart, SegEnd, SegT);
					const float DistToSegSq = FVector::DistSquared(Center, ClosestPt);
					const float HalfWidth = 0.5f*Lane.Width;

					// Check if the radius fully overlaps the lane width (computed as a lane radius).
					if (Radius > HalfWidth && DistToSegSq < FMath::Square(Radius - HalfWidth))
					{
						FZoneGraphLaneSection& Section = OutLaneSections.Emplace_GetRef();
						Section.LaneHandle = FZoneGraphLaneHandle(uint32(LaneIdx), Storage.DataHandle);

						// Approximate the section size using the full radius.
						const float DistAlongLane =  FMath::Lerp(Storage.LanePointProgressions[i], Storage.LanePointProgressions[i + 1], SegT);
						Section.StartDistanceAlongLane = FMath::Max(0.f, DistAlongLane - Radius);
						Section.EndDistanceAlongLane = FMath::Min(DistAlongLane + Radius, Storage.LanePointProgressions[Lane.GetLastPoint()]);

						break;
					}
				}
			}
		}
	});

	return OutLaneSections.Num() > 0;
}
	
} // UE::ZoneGraph::Query
