// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZoneGraphRenderingComponent.h"
#include "ZoneGraphData.h"
#include "ZoneGraphTypes.h"
#include "ZoneGraphQuery.h"
#include "ZoneGraphRenderingUtilities.h"
#include "ZoneGraphSettings.h"
#include "Engine/CollisionProfile.h"
#include "Engine/GameViewportClient.h"
#include "PrimitiveViewRelevance.h"
#include "Engine/Engine.h"
#include "TimerManager.h"

#if WITH_EDITOR
#include "Editor.h"
#include "EditorViewportClient.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ZoneGraphRenderingComponent)

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
#include "Engine/Canvas.h"
#include "Debug/DebugDrawService.h"

namespace UE::ZoneGraph::Debug {

bool bDisplayLaneIndex = false;
bool bDisplayLaneEntryID = false;
bool bDisplayLaneTags = false;
bool bDisplayZoneBVTree = false;

FAutoConsoleVariableRef Vars[] = {
	FAutoConsoleVariableRef(TEXT("ai.debug.zonegraph.DisplayLaneIndex"), bDisplayLaneIndex, TEXT("Adds text to represent ID above each zone graph lane.")),
	FAutoConsoleVariableRef(TEXT("ai.debug.zonegraph.DisplayLaneEntryID"), bDisplayLaneEntryID, TEXT("Adds text to represent entry ID above each zone graph lane.")),
	FAutoConsoleVariableRef(TEXT("ai.debug.zonegraph.DisplayLaneTags"), bDisplayLaneTags, TEXT("Adds text to represent all lane tags as name list + mask (in hex value).")),
	FAutoConsoleVariableRef(TEXT("ai.debug.zonegraph.DisplayZoneBVTree"), bDisplayZoneBVTree, TEXT("Displays zone BV-tree.")),
};

} // UE::ZoneGraph::Debug
#endif // !UE_BUILD_SHIPPING && !UE_BUILD_TEST

//////////////////////////////////////////////////////////////////////////
// FZoneGraphSceneProxy

SIZE_T FZoneGraphSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

FZoneGraphSceneProxy::FZoneGraphSceneProxy(const UPrimitiveComponent& InComponent, const AZoneGraphData& ZoneGraph)
	: FDebugRenderSceneProxy(&InComponent)
{
	DrawType = EDrawType::SolidAndWireMeshes;

	WeakRenderingComponent = MakeWeakObjectPtr(const_cast<UZoneGraphRenderingComponent*>(Cast<UZoneGraphRenderingComponent>(&InComponent)));
	bSkipDistanceCheck = GIsEditor && (GEngine->GetDebugLocalPlayer() == nullptr);

}

FZoneGraphSceneProxy::~FZoneGraphSceneProxy()
{
}

void FZoneGraphSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
	FDebugRenderSceneProxy::GetDynamicMeshElements(Views, ViewFamily, VisibilityMap, Collector);

	UZoneGraphRenderingComponent* RenderingComp = WeakRenderingComponent.Get();
	const AZoneGraphData* ZoneGraphData = RenderingComp ? Cast<AZoneGraphData>(RenderingComp->GetOwner()) : nullptr;
	if (!ZoneGraphData)
	{
		return;
	}

	FScopeLock RegistrationLock(&(ZoneGraphData->GetStorageLock()));
	const FZoneGraphStorage& ZoneStorage = ZoneGraphData->GetStorage();

	static const float DepthBias = 0.0001f;	// Little bias helps to make the lines visible when directly on top of geometry.
	static const float LaneLineThickness = 2.0f;
	static const float BoundaryLineThickness = 0.0f;

	const FDrawDistances Distances = GetDrawDistances(GetMinDrawDistance(), GetMaxDrawDistance());
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			const FSceneView* View = Views[ViewIndex];
			if (!ShouldRenderZoneGraph(*View))
			{
				continue;
			}
			FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);

			const FVector Origin = View->ViewMatrices.GetViewOrigin();

			for (int i = 0; i < ZoneStorage.Zones.Num(); i++)
			{
				const FZoneData& Zone = ZoneStorage.Zones[i];
				const FZoneVisibility DrawInfo = CalculateZoneVisibility(Distances, Origin, Zone.Bounds.GetCenter());

				if (!DrawInfo.bVisible)
				{
					continue;
				}

				// Draw boundary
				UE::ZoneGraph::RenderingUtilities::DrawZoneBoundary(ZoneStorage, i, PDI, FMatrix::Identity, BoundaryLineThickness, DepthBias, DrawInfo.Alpha);
				// Draw Lanes
				UE::ZoneGraph::RenderingUtilities::DrawZoneLanes(ZoneStorage, i, PDI, FMatrix::Identity, LaneLineThickness, DepthBias, DrawInfo.Alpha, DrawInfo.bDetailsVisible);
			}
			
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
			if (UE::ZoneGraph::Debug::bDisplayZoneBVTree)
			{
				const FColor NodeColor(64,64,64);
				const FColor LeafNodeColor(255,64,0);

				const FVector ViewExt(FMath::Sqrt(Distances.MaxDrawDistanceSqr));
				FZoneGraphBVNode ViewBounds = ZoneStorage.ZoneBVTree.CalcNodeBounds(FBox(Origin - ViewExt, Origin + ViewExt));
				
				TConstArrayView<FZoneGraphBVNode> Nodes = ZoneStorage.ZoneBVTree.GetNodes();
				for (const FZoneGraphBVNode& Node : Nodes)
				{
					if (ViewBounds.DoesOverlap(Node))
					{
						const FColor Color = Node.Index >= 0 ? LeafNodeColor : NodeColor;
						const float Thickness = Node.Index >= 0 ? 2.0f : 1.0f;
						const FBox NodeWorldBounds = ZoneStorage.ZoneBVTree.CalcWorldBounds(Node);
						DrawWireBox(PDI, NodeWorldBounds, Color, SDPG_World, Thickness, 0, /*bScreenSpace*/true);
					}
				}
			}
#endif // !UE_BUILD_SHIPPING && !UE_BUILD_TEST
		}
	}
}

FPrimitiveViewRelevance FZoneGraphSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	const bool bVisible = !!View->Family->EngineShowFlags.Navigation;
	FPrimitiveViewRelevance Result;
	Result.bDrawRelevance = bVisible && IsShown(View);
	Result.bDynamicRelevance = true;
	// ideally the TranslucencyRelevance should be filled out by the material, here we do it conservative
	Result.bSeparateTranslucency = Result.bNormalTranslucency = bVisible && IsShown(View);
	return Result;
}

FZoneGraphSceneProxy::FDrawDistances FZoneGraphSceneProxy::GetDrawDistances(const float MinDrawDistance, const float MaxDrawDistance)
{
	float ShapeMaxDrawDistance = 0.5f;
	if (const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>())
	{
		ShapeMaxDrawDistance = ZoneGraphSettings->GetShapeMaxDrawDistance();
	}

	const float CombinedMaxDrawDistance = FMath::Min(ShapeMaxDrawDistance, MaxDrawDistance);

	FDrawDistances Distances;
	Distances.MinDrawDistanceSqr = FMath::Square(MinDrawDistance);
	Distances.MaxDrawDistanceSqr = FMath::Square(CombinedMaxDrawDistance);
	Distances.FadeDrawDistanceSqr = FMath::Square(CombinedMaxDrawDistance * 0.9f);
	Distances.DetailDrawDistanceSqr = FMath::Square(CombinedMaxDrawDistance * 0.5f);

	return Distances;
}

FZoneGraphSceneProxy::FZoneVisibility FZoneGraphSceneProxy::CalculateZoneVisibility(const FZoneGraphSceneProxy::FDrawDistances& Distances, const FVector Origin, const FVector Position)
{
	FZoneVisibility DrawInfo;

	// Taking into account the min and maximum drawing distance
	const float DistanceSqr = FVector::DistSquared(Position, Origin);
	DrawInfo.bVisible = !(DistanceSqr < Distances.MinDrawDistanceSqr || DistanceSqr > Distances.MaxDrawDistanceSqr);

	// Only compute other informations if we actually draw something
	if (DrawInfo.bVisible)
	{
		// Only draw details close to camera
		DrawInfo.bDetailsVisible = DistanceSqr < Distances.DetailDrawDistanceSqr;
		// Fade visualization before culling.
		DrawInfo.Alpha = 1.0f - FMath::Clamp((DistanceSqr - Distances.FadeDrawDistanceSqr) / (Distances.MaxDrawDistanceSqr - Distances.FadeDrawDistanceSqr), 0.0f, 1.0f);
	}

	return DrawInfo;
}

bool FZoneGraphSceneProxy::ShouldRenderZoneGraph(const FSceneView& View)
{
	return !!View.Family->EngineShowFlags.Navigation;
}

//////////////////////////////////////////////////////////////////////////
// UZoneGraphRenderingComponent

UZoneGraphRenderingComponent::UZoneGraphRenderingComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	bSelectable = false;
	bPreviousShowNavigation = false;
}

FPrimitiveSceneProxy* UZoneGraphRenderingComponent::CreateSceneProxy()
{
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	FZoneGraphSceneProxy* ZoneGraphSceneProxy = nullptr;
	const AZoneGraphData* ZoneGraphData = Cast<AZoneGraphData>(GetOwner());
	if (ZoneGraphData != nullptr && IsVisible() && IsNavigationShowFlagSet(GetWorld()))
	{
		ZoneGraphSceneProxy = new FZoneGraphSceneProxy(*this, *ZoneGraphData);
	}

	return ZoneGraphSceneProxy;
#else
	return nullptr;
#endif //!UE_BUILD_SHIPPING && !UE_BUILD_TEST
}

void UZoneGraphRenderingComponent::OnRegister()
{
	Super::OnRegister();

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	DebugTextDrawingDelegateHandle = UDebugDrawService::Register(TEXT("ZoneGraph"),
		FDebugDrawDelegate::CreateLambda([this](UCanvas* Canvas, APlayerController*)
			{
				if (!UE::ZoneGraph::Debug::bDisplayLaneIndex && !UE::ZoneGraph::Debug::bDisplayLaneEntryID && !UE::ZoneGraph::Debug::bDisplayLaneTags)
				{
					return;
				}

				const FSceneView* View = Canvas->SceneView;
				if (!FZoneGraphSceneProxy::ShouldRenderZoneGraph(*View))
				{
					return;
				}

				TGuardValue<FColor> ColorGuard(Canvas->DrawColor, FColor::White);
				const UFont* Font = GEngine->GetSmallFont();
				const AZoneGraphData* ZoneGraphData = Cast<AZoneGraphData>(GetOwner());
				const FZoneGraphStorage& ZoneStorage = ZoneGraphData->GetStorage();
				// * A CachedMaxDrawDistance of 0 indicates that the primitive should not be culled by distance.
				const float MaxDrawDistance = CachedMaxDrawDistance > 0 ? CachedMaxDrawDistance : FLT_MAX;
				const FZoneGraphSceneProxy::FDrawDistances Distances = FZoneGraphSceneProxy::GetDrawDistances(MinDrawDistance, MaxDrawDistance);
				const FVector Origin = View->ViewMatrices.GetViewOrigin();

				for (const FZoneData& Zone : ZoneStorage.Zones)
				{
					const FZoneGraphSceneProxy::FZoneVisibility DrawInfo = FZoneGraphSceneProxy::CalculateZoneVisibility(Distances, Origin, Zone.Bounds.GetCenter());
					if (!DrawInfo.bVisible || !DrawInfo.bDetailsVisible)
					{
						continue;
					}

					for (int32 LaneIdx = Zone.LanesBegin; LaneIdx < Zone.LanesEnd; LaneIdx++)
					{
						if (UE::ZoneGraph::Debug::bDisplayLaneIndex || UE::ZoneGraph::Debug::bDisplayLaneTags)
						{
							FZoneGraphLaneLocation CenterLoc;
							UE::ZoneGraph::Query::CalculateLocationAlongLaneFromRatio(ZoneStorage, LaneIdx, 0.5f, CenterLoc);
							const FVector Location = FVector(GetComponentTransform().TransformPosition(CenterLoc.Position));
							const FVector ScreenLoc = Canvas->Project(Location);

							FString Text;
							// Display Index + Tags
							if (UE::ZoneGraph::Debug::bDisplayLaneIndex && UE::ZoneGraph::Debug::bDisplayLaneTags)
							{
								const FZoneGraphTagMask Mask = ZoneStorage.Lanes[LaneIdx].Tags;
								Text = FString::Printf(TEXT("%d\n%s\n0x%08X"), LaneIdx, *UE::ZoneGraph::Helpers::GetTagMaskString(Mask, TEXT(", ")), Mask.GetValue());
							}
							// Display Index only
							else if (UE::ZoneGraph::Debug::bDisplayLaneIndex)
							{
								Text = FString::Printf(TEXT("%d"), LaneIdx);
							}
							// Display Tags only
							else // UE::ZoneGraph::Debug::bDisplayLaneTags
							{
								const FZoneGraphTagMask Mask = ZoneStorage.Lanes[LaneIdx].Tags;
								Text = FString::Printf(TEXT("%s\n0x%08X"), *UE::ZoneGraph::Helpers::GetTagMaskString(Mask, TEXT(", ")), Mask.GetValue());
							}

							Canvas->DrawText(Font, Text, ScreenLoc.X, ScreenLoc.Y);
						}
						if (UE::ZoneGraph::Debug::bDisplayLaneEntryID)
						{
							const FZoneLaneData& Lane = ZoneStorage.Lanes[LaneIdx];
							const FVector StartLocalPos = ZoneStorage.LanePoints[Lane.PointsBegin];
							const FVector StartLocalDir = (ZoneStorage.LanePoints[Lane.PointsBegin + 1] - StartLocalPos).GetSafeNormal();
							const FVector EndLocalPos = ZoneStorage.LanePoints[Lane.PointsEnd - 1];
							const FVector EndLocalDir = (ZoneStorage.LanePoints[Lane.PointsEnd - 2] - EndLocalPos).GetSafeNormal();
							static const float Offset = Lane.Width * 0.1f;
							const FVector StartPos = FVector(GetComponentTransform().TransformPosition(StartLocalPos + StartLocalDir * Offset));
							const FVector EndPos = FVector(GetComponentTransform().TransformPosition(EndLocalPos + EndLocalDir * Offset));
							const FVector StartScreenLoc = Canvas->Project(StartPos);
							const FVector EndScreenLoc = Canvas->Project(EndPos);
							Canvas->DrawText(Font, FString::Printf(TEXT("S%d"), Lane.StartEntryId), StartScreenLoc.X, StartScreenLoc.Y);
							Canvas->DrawText(Font, FString::Printf(TEXT("E%d"), Lane.EndEntryId), EndScreenLoc.X, EndScreenLoc.Y);
						}
					}
				}
			}));

	// it's a kind of HACK but there is no event or other information that show flag was changed by user => we have to check it periodically
#if WITH_EDITOR
	if (GEditor)
	{
		GEditor->GetTimerManager()->SetTimer(TimerHandle, FTimerDelegate::CreateUObject(this, &UZoneGraphRenderingComponent::CheckDrawFlagTimerFunction), 1, true);
	}
	else
#endif //WITH_EDITOR
	{
		GetWorld()->GetTimerManager().SetTimer(TimerHandle, FTimerDelegate::CreateUObject(this, &UZoneGraphRenderingComponent::CheckDrawFlagTimerFunction), 1, true);
	}
#endif //!UE_BUILD_SHIPPING && !UE_BUILD_TEST
}

void UZoneGraphRenderingComponent::OnUnregister()
{
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	UDebugDrawService::Unregister(DebugTextDrawingDelegateHandle);

	// it's a kind of HACK but there is no event or other information that show flag was changed by user => we have to check it periodically
#if WITH_EDITOR
	if (GEditor)
	{
		GEditor->GetTimerManager()->ClearTimer(TimerHandle);
	}
	else
#endif //WITH_EDITOR
	{
		GetWorld()->GetTimerManager().ClearTimer(TimerHandle);
	}
#endif //!UE_BUILD_SHIPPING && !UE_BUILD_TEST
	Super::OnUnregister();
}

FBoxSphereBounds UZoneGraphRenderingComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FBox BoundingBox(ForceInit);

	const AZoneGraphData* ZoneGraphData = Cast<AZoneGraphData>(GetOwner());
	if (ZoneGraphData)
	{
		BoundingBox = ZoneGraphData->GetBounds();
	}
	return FBoxSphereBounds(BoundingBox);
}

bool UZoneGraphRenderingComponent::IsNavigationShowFlagSet(const UWorld* World)
{
	bool bShowNavigation = false;

	FWorldContext* WorldContext = GEngine->GetWorldContextFromWorld(World);

#if WITH_EDITOR
	if (GEditor && WorldContext && WorldContext->WorldType != EWorldType::Game)
	{
		bShowNavigation = WorldContext->GameViewport != nullptr && WorldContext->GameViewport->EngineShowFlags.Navigation;
		if (bShowNavigation == false)
		{
			// we have to check all viewports because we can't to distinguish between SIE and PIE at this point.
			for (FEditorViewportClient* CurrentViewport : GEditor->GetAllViewportClients())
			{
				if (CurrentViewport && CurrentViewport->EngineShowFlags.Navigation)
				{
					bShowNavigation = true;
					break;
				}
			}
		}
	}
	else
#endif //WITH_EDITOR
	{
		bShowNavigation = WorldContext && WorldContext->GameViewport && WorldContext->GameViewport->EngineShowFlags.Navigation;
	}

	return bShowNavigation; 
}

void UZoneGraphRenderingComponent::CheckDrawFlagTimerFunction()
{
	const UWorld* World = GetWorld();

	const bool bShowNavigation = bForceUpdate || IsNavigationShowFlagSet(World);

	if (bShowNavigation && bPreviousShowNavigation != bShowNavigation)
	{
		bForceUpdate = false;
		bPreviousShowNavigation = bShowNavigation;
		MarkRenderStateDirty();
	}
}
