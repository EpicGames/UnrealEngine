// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZoneGraphRenderingUtilities.h"
#include "Curves/BezierUtilities.h"
#include "PrimitiveSceneProxy.h"
#include "ZoneGraphTypes.h"
#include "ZoneGraphQuery.h"
#include "ZoneGraphSettings.h"
#include "Misc/AssertionMacros.h"
#include "DebugRenderSceneProxy.h"

namespace UE::ZoneGraph::RenderingUtilities
{

FColor GetTagMaskColor(const FZoneGraphTagMask TagMask, TConstArrayView<FZoneGraphTagInfo> TagInfos)
{
	// Pick the first color
	FLinearColor Color = FLinearColor::Black;
	float Weight = 0.0f;
	for (const FZoneGraphTagInfo& Info : TagInfos)
	{
		if (TagMask.Contains(Info.Tag))
		{
			Color += FLinearColor(Info.Color);
			Weight += 1.0f;
		}
	}

	return Weight > 0.0f ? (Color / Weight).ToFColor(true /*bSRGB*/) : FColor::Black;
}

void DrawZoneBoundary(const FZoneGraphStorage& ZoneStorage, int32 ZoneIndex, FPrimitiveDrawInterface* PDI, const FMatrix& LocalToWorld, const float LineThickness, const float DepthBias, const float Alpha, const FZoneDrawAnnotator& Annotator)
{
	if (ZoneIndex < 0 || ZoneIndex >= ZoneStorage.Zones.Num())
	{
		return;
	}

	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
	check(ZoneGraphSettings);
	FZoneGraphTagMask VisTagMask = Annotator.VisualizedTagsOverride.Get(ZoneGraphSettings->GetVisualizedTags());
	TConstArrayView<FZoneGraphTagInfo>TagInfo = ZoneGraphSettings->GetTagInfos();

	const FZoneData& Zone = ZoneStorage.Zones[ZoneIndex];
	const bool bTransform = !LocalToWorld.Equals(FMatrix::Identity);
	FColor Color = GetTagMaskColor(Annotator.ZoneTagsOverride.Get(Zone.Tags) & VisTagMask, TagInfo);
	Color.A = uint8(128 * Alpha);

	FVector PrevPoint = bTransform ? FVector(LocalToWorld.TransformPosition(ZoneStorage.BoundaryPoints[Zone.BoundaryPointsEnd - 1])) : ZoneStorage.BoundaryPoints[Zone.BoundaryPointsEnd - 1];
	for (int32 i = Zone.BoundaryPointsBegin; i < Zone.BoundaryPointsEnd; i++)
	{
		const FVector Point = bTransform ? FVector(LocalToWorld.TransformPosition(ZoneStorage.BoundaryPoints[i])) : ZoneStorage.BoundaryPoints[i];
		PDI->DrawTranslucentLine(PrevPoint, Point, Color, SDPG_World, LineThickness, 0/*DepthBias*/, true);
		PrevPoint = Point;
	}
}

void DrawLane(const FZoneGraphStorage& ZoneStorage, FPrimitiveDrawInterface* PDI, const FZoneGraphLaneHandle& LaneHandle, const FColor Color, const float LineThickness /*= 5.f*/, const FVector& Offset /*= FVector::ZeroVector*/)
{
	if (!ensure(LaneHandle.IsValid()))
	{
		return;
	}

	const FVector OffsetZ(0.f, 0.f, 0.1f);
	const FZoneLaneData& Lane = ZoneStorage.Lanes[LaneHandle.Index];
	FVector PrevPoint = ZoneStorage.LanePoints[Lane.PointsBegin] + Offset;
	for (int32 i = Lane.PointsBegin + 1; i < Lane.PointsEnd; i++)
	{
		const FVector Point = ZoneStorage.LanePoints[i] + Offset;
		PDI->DrawLine(PrevPoint + OffsetZ, Point + OffsetZ, Color, SDPG_World, LineThickness);
		PrevPoint = Point;
	}
}

void DrawLane(const FZoneGraphStorage& ZoneStorage, FPrimitiveDrawInterface* PDI,
			  const FZoneGraphLaneLocation& InStartLocation, const FZoneGraphLaneLocation& InEndLocation,
			  const FColor Color, const float LineThickness /*= 5.f*/, const FVector& Offset)
{
	FZoneGraphLaneLocation StartLocation = InStartLocation;
	FZoneGraphLaneLocation EndLocation = InEndLocation;

	// At least one location must be valid
	ensure(StartLocation.LaneHandle.IsValid() || EndLocation.LaneHandle.IsValid());
	const FZoneLaneData& Lane = StartLocation.IsValid() ? ZoneStorage.Lanes[StartLocation.LaneHandle.Index] : ZoneStorage.Lanes[EndLocation.LaneHandle.Index];

	// If both are valid, they must be on the same lane
	if (StartLocation.LaneHandle.IsValid() && EndLocation.LaneHandle.IsValid())
	{
		ensure(StartLocation.LaneHandle.Index == EndLocation.LaneHandle.Index);

		if (InStartLocation.DistanceAlongLane > InEndLocation.DistanceAlongLane)
		{
			// Swap
			StartLocation = InEndLocation;
			EndLocation   = InStartLocation;
		}
	}

	const FVector OffsetZ = Offset + FVector(0.f, 0.f, 0.1f);
	const int32 PointsBegin = StartLocation.LaneHandle.IsValid() ? StartLocation.LaneSegment : Lane.PointsBegin;
	const int32 PointsEnd = EndLocation.LaneHandle.IsValid() ? EndLocation.LaneSegment + 1 : Lane.PointsEnd;

	FVector PrevPoint = StartLocation.LaneHandle.IsValid() ? StartLocation.Position : ZoneStorage.LanePoints[Lane.PointsBegin];
	for (int32 i = PointsBegin + 1; i < PointsEnd; i++)
	{
		const FVector Point = ZoneStorage.LanePoints[i];
		PDI->DrawLine(PrevPoint + OffsetZ, Point + OffsetZ, Color, SDPG_World, LineThickness);
		PrevPoint = Point;
	}

	if (EndLocation.LaneHandle.IsValid())
	{
		// Last (partial segment)
		PDI->DrawLine(PrevPoint + OffsetZ, EndLocation.Position + OffsetZ, Color, SDPG_World, LineThickness, /*screenspace*/true);
	}
}

void DrawLanes(const FZoneGraphStorage& ZoneStorage, FPrimitiveDrawInterface* PDI, const TArray<FZoneGraphLaneHandle>& Lanes, const FColor Color, const float LineThickness /*= 5.f*/, const FVector& Offset /*= FVector::ZeroVector*/)
{
	for (const FZoneGraphLaneHandle& LaneHandle : Lanes)
	{
		DrawLane(ZoneStorage, PDI, LaneHandle, Color, LineThickness, Offset);
	}
}

static FLinearColor GetLinkColor(const FZoneGraphLinkedLane& LinkedLane)
{
	if (LinkedLane.Type == EZoneLaneLinkType::Adjacent)
	{
		if (LinkedLane.HasFlags(EZoneLaneLinkFlags::Left))
		{
			const float Scale = LinkedLane.HasFlags(EZoneLaneLinkFlags::OppositeDirection) ? 1.0f : 0.5f;
			return FLinearColor::Green * Scale;
		}
		else if (LinkedLane.HasFlags(EZoneLaneLinkFlags::Right))
		{
			const float Scale = LinkedLane.HasFlags(EZoneLaneLinkFlags::OppositeDirection) ? 1.0f : 0.5f;
			return FLinearColor::Red * Scale;
		}
		else if (LinkedLane.HasFlags(EZoneLaneLinkFlags::Merging))
		{
			return FLinearColor::Yellow;
		}
		else if (LinkedLane.HasFlags(EZoneLaneLinkFlags::Splitting))
		{
			return FMath::Lerp(FLinearColor::Yellow, FLinearColor::Red, 0.3f);
		}
	}
	else if (LinkedLane.Type == EZoneLaneLinkType::Outgoing)
	{
		return FLinearColor::Blue;
	}
	else if (LinkedLane.Type == EZoneLaneLinkType::Incoming)
	{
		return FMath::Lerp(FLinearColor::Blue, FLinearColor::Red, 0.3f);
	}
	return FLinearColor::Black;
}

void DrawLinkedLanes(const FZoneGraphStorage& ZoneStorage, FPrimitiveDrawInterface* PDI, const FZoneGraphLaneHandle& SourceLaneHandle, const TArray<FZoneGraphLinkedLane>& LinkedLanes, const float LineThickness /*= 5.f*/)
{
	static const float OffsetIncrement = 10.0f;
	FVector Offset(0, 0, OffsetIncrement);
	for (const FZoneGraphLinkedLane& LinkedLane : LinkedLanes)
	{
		// Draw arc arrow between linked lanes.
		FLinearColor LinkColor = GetLinkColor(LinkedLane);
		DrawLane(ZoneStorage, PDI, LinkedLane.DestLane, LinkColor.ToFColor(/*SRGB*/true), LineThickness, Offset);
		Offset.Z += OffsetIncrement;
	}
}

void DrawLanePath(const FZoneGraphStorage& ZoneStorage, FPrimitiveDrawInterface* PDI, const FZoneGraphLanePath& LanePath, const FColor Color, const float LineThickness /*= 5.f*/)
{
	// Draw start and end locations
	const float PointSize = 1.5f;
	PDI->DrawPoint(LanePath.StartLaneLocation.Position, FColor::Green, PointSize*LineThickness, SDPG_World);
	PDI->DrawPoint(LanePath.EndLaneLocation.Position, FColor::Red, PointSize*LineThickness, SDPG_World);

	FZoneGraphLaneLocation CurLocation = LanePath.StartLaneLocation;
	if (!CurLocation.IsValid())
	{
		// LanePath is not set properly
		return;
	}

	const FVector OffsetZ(0.f, 0.f, 0.1f);
	const int32 LanePathCount = LanePath.Lanes.Num();
	const int32 LastLanePathIndex = LanePathCount - 1;
	for (int32 LanePathIndex = 0; LanePathIndex < LanePathCount; ++LanePathIndex)
	{
		// If the next lane is in the same zone, draw a line to it to show that we are changing lane
		if (LanePathIndex+1 < LanePathCount)
		{
			const int32 CurLaneIndex = LanePath.Lanes[LanePathIndex].Index;
			const int32 NextLaneIndex = LanePath.Lanes[LanePathIndex+1].Index;
			const int32 CurZoneIndex = ZoneStorage.Lanes[CurLaneIndex].ZoneIndex;
			const int32 NextZoneIndex = ZoneStorage.Lanes[NextLaneIndex].ZoneIndex;
			if (CurZoneIndex == NextZoneIndex)
			{
				float Width = 0.f;
				UE::ZoneGraph::Query::GetLaneWidth(ZoneStorage, CurLaneIndex, Width);
				const float SearchDistance = 5.f * Width; // Arbitrary search dist
				FZoneGraphLaneLocation FoundLocation;
				float DistanceSqr = 0.f;
				FBox Bounds(CurLocation.Position, CurLocation.Position);
				Bounds = Bounds.ExpandBy(SearchDistance);
				const bool bFound = UE::ZoneGraph::Query::FindNearestLocationOnLane( // Find nearest location on the next lane
					ZoneStorage,
					FZoneGraphLaneHandle(NextLaneIndex, ZoneStorage.DataHandle),
					Bounds,
					FoundLocation,
					DistanceSqr
				);

				if (bFound)
				{
					PDI->DrawLine(CurLocation.Position + OffsetZ, FoundLocation.Position + OffsetZ, Color, SDPG_World, LineThickness, /*screenspace*/true);
					CurLocation = FoundLocation;
					continue;
				}
				else
				{
					// Something is wrong, just stop displaying the path
					break;
				}
			}
		}

		if (CurLocation.IsValid())
		{
			// If we have a CurLocation, draw from the CurLocation location to the end of the lane
			FZoneGraphLaneLocation TempLocation;
			TempLocation.Reset();
			if (LanePathIndex == LastLanePathIndex)
			{
				TempLocation = LanePath.EndLaneLocation;
			}

			// Draw the lane between cur and temp locations
			DrawLane(ZoneStorage, PDI, CurLocation, TempLocation, Color, LineThickness);
			CurLocation.Reset(); // Clear CurLocation for next iterations
		}
		else if (LanePathIndex < LastLanePathIndex)
		{
			// No current location, draw the full lane
			DrawLane(ZoneStorage, PDI, LanePath.Lanes[LanePathIndex], Color, LineThickness);
		}
		else
		{
			// Last lane of the path, draw from the begining of the lane to the end location
			check(LanePathIndex == LastLanePathIndex);
			const FZoneGraphLaneLocation& Location = LanePath.EndLaneLocation;
			if (!ensure(Location.LaneHandle.IsValid()))
			{
				break;
			}

			const int32 SegEnd = Location.LaneSegment;
			const FZoneLaneData& Lane = ZoneStorage.Lanes[Location.LaneHandle.Index];
			FVector PrevPoint = ZoneStorage.LanePoints[Lane.PointsBegin];
			for (int32 SegIndex = Lane.PointsBegin + 1; SegIndex < SegEnd; SegIndex++)
			{
				const FVector Point = ZoneStorage.LanePoints[SegIndex];
					PDI->DrawLine(PrevPoint + OffsetZ, Point + OffsetZ, Color, SDPG_World, LineThickness, /*screenspace*/true);
				PrevPoint = Point;
			}

			// Last (partial segment)
			PDI->DrawLine(PrevPoint + OffsetZ, Location.Position + OffsetZ, Color, SDPG_World, LineThickness, /*screenspace*/true);
		}
	}
}

void DrawZoneLanes(const FZoneGraphStorage& ZoneStorage, int32 ZoneIndex, FPrimitiveDrawInterface* PDI, const FMatrix& LocalToWorld, const float LineThickness, const float DepthBias, const float Alpha, const bool bDrawDetails, const FLaneHighlight& LaneHighlight, const FZoneDrawAnnotator& Annotator)
{
	if (ZoneIndex < 0 || ZoneIndex >= ZoneStorage.Zones.Num())
	{
		return;
	}

	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
	check(ZoneGraphSettings);
	FZoneGraphTagMask VisTagMask = Annotator.VisualizedTagsOverride.Get(ZoneGraphSettings->GetVisualizedTags());
	TConstArrayView<FZoneGraphTagInfo>TagInfo = ZoneGraphSettings->GetTagInfos();

	const FZoneData& Zone = ZoneStorage.Zones[ZoneIndex];
	const bool bTransform = !LocalToWorld.Equals(FMatrix::Identity);
	const bool bHasAlpha = Alpha > 0.0f;

	for (int32 LaneIdx = Zone.LanesBegin; LaneIdx < Zone.LanesEnd; LaneIdx++)
	{
		const FZoneLaneData& Lane = ZoneStorage.Lanes[LaneIdx];
		FColor Color = GetTagMaskColor((Annotator.GetLaneTagsOverride.IsSet() ? Annotator.GetLaneTagsOverride.GetValue()(LaneIdx, Lane) : Lane.Tags) & VisTagMask, TagInfo);
		bool bDrawTransparent = false;

		FVector PrevPoint = bTransform ? FVector(LocalToWorld.TransformPosition(ZoneStorage.LanePoints[Lane.PointsBegin])) : ZoneStorage.LanePoints[Lane.PointsBegin];

		if (LaneHighlight.IsValid())
		{
			// Dim lanes which are not close to the highlighted point.
			// The highlight shape and the lanes do not necessarily match exactly (due to snapping), so we'll give a little slack in the test.
			static const float Threshold = 50.0f;
			const float ThresholdX = Threshold;
			const float ThresholdY = LaneHighlight.Width + Threshold;
			const float ThresholdZ = LaneHighlight.Width + Threshold;
			const FVector LocalPosition = LaneHighlight.Rotation.UnrotateVector(PrevPoint - LaneHighlight.Position).GetAbs();
			const bool bInside = LocalPosition.X < ThresholdX && LocalPosition.Y < ThresholdY && LocalPosition.Z < ThresholdZ;
			if (!bInside)
			{
				Color.A = 64;
			}
			bDrawTransparent = true;
		}

		if (bHasAlpha)
		{
			Color.A = uint8(Color.A * Alpha);
			bDrawTransparent = true;
		}

		if (bDrawTransparent)
		{
			for (int32 i = Lane.PointsBegin + 1; i < Lane.PointsEnd; i++)
			{
				const FVector Point = bTransform ? FVector(LocalToWorld.TransformPosition(ZoneStorage.LanePoints[i])) : ZoneStorage.LanePoints[i];
				PDI->DrawTranslucentLine(PrevPoint, Point, Color, SDPG_World, LineThickness, DepthBias, true);
				PrevPoint = Point;
			}
		}
		else
		{
			for (int32 i = Lane.PointsBegin + 1; i < Lane.PointsEnd; i++)
			{
				const FVector Point = bTransform ? FVector(LocalToWorld.TransformPosition(ZoneStorage.LanePoints[i])) : ZoneStorage.LanePoints[i];
				PDI->DrawLine(PrevPoint, Point, Color, SDPG_World, LineThickness, DepthBias, true);
				PrevPoint = Point;
			}
		}

		// Details do not take alpha into account. Alpha is used mainly to fade out the main body of the zone before it disappears.
		// Details are assumed to be culled before that.
		if (bDrawDetails)
		{
			// Draw direction arrows, one at each end, or one at middle for short lanes.
			float LaneLength = 0.0f;
			UE::ZoneGraph::Query::GetLaneLength(ZoneStorage, LaneIdx, LaneLength);
			const int NumArrows = LaneLength > Lane.Width * 1.5f ? 2 : 1;
			const float ArrowDist = FMath::Min(Lane.Width * 0.5f, LaneLength / 2.0f);
			const float ArrowSize = Lane.Width * 0.2f;

			for (int i = 0; i < NumArrows; i++)
			{
				FZoneGraphLaneLocation ArrowLoc;
				UE::ZoneGraph::Query::CalculateLocationAlongLane(ZoneStorage, LaneIdx, i == 0 ? ArrowDist : (LaneLength - ArrowDist), ArrowLoc);
				const FVector ArrowPos = bTransform ? FVector(LocalToWorld.TransformPosition(ArrowLoc.Position)) : ArrowLoc.Position;
				const FVector ArrowDir = bTransform ? FVector(LocalToWorld.TransformVector(ArrowLoc.Direction)) : ArrowLoc.Direction;
				const FVector ArrowOrigin = ArrowPos;
				const FVector ArrowTip = ArrowPos + ArrowDir * ArrowSize;

				FPrimitiveSceneProxy::DrawArrowHead(PDI, ArrowTip, ArrowOrigin, ArrowSize, Color, SDPG_World, LineThickness, true);
			}

			// Draw adjacent lanes
			FZoneGraphLaneLocation StartLoc;
			UE::ZoneGraph::Query::CalculateLocationAlongLane(ZoneStorage, LaneIdx, LaneLength * 0.3f, StartLoc);
			const FVector LaneStartPoint = bTransform ? FVector(LocalToWorld.TransformPosition(StartLoc.Position)) : StartLoc.Position;
			const FVector LaneStartDir = bTransform ? FVector(LocalToWorld.TransformVector(StartLoc.Direction)) : StartLoc.Direction;
			const FVector LaneStartSide = FVector::CrossProduct(LaneStartDir, FVector::UpVector);

			// Draw small ticks towards the adjacent lane.
			FZoneGraphLinkedLane LeftLinkedLane;
			if (UE::ZoneGraph::Query::GetFirstLinkedLane(ZoneStorage, LaneIdx, EZoneLaneLinkType::Adjacent, EZoneLaneLinkFlags::Left, EZoneLaneLinkFlags::None, LeftLinkedLane) && LeftLinkedLane.IsValid())
			{
				PDI->DrawLine(LaneStartPoint, LaneStartPoint + LaneStartSide * Lane.Width * 0.1f, FMath::Lerp(FLinearColor(Color), FLinearColor::Green, 0.3f), SDPG_World, LineThickness, DepthBias, true);
			}
			FZoneGraphLinkedLane RightLinkedLane;
			if (UE::ZoneGraph::Query::GetFirstLinkedLane(ZoneStorage, LaneIdx, EZoneLaneLinkType::Adjacent, EZoneLaneLinkFlags::Right, EZoneLaneLinkFlags::None, RightLinkedLane) && RightLinkedLane.IsValid())
			{
				PDI->DrawLine(LaneStartPoint, LaneStartPoint - LaneStartSide * Lane.Width * 0.1f, FMath::Lerp(FLinearColor(Color), FLinearColor::Red, 0.3f), SDPG_World, LineThickness, DepthBias, true);
			}
		}
	}
}

void DrawZoneShapeConnectors(TConstArrayView<FZoneShapeConnector> Connectors, TConstArrayView<FZoneShapeConnection> Connections, FPrimitiveDrawInterface* PDI, const FMatrix& LocalToWorld, const float DepthBias)
{
	for (int32 i = 0; i < Connectors.Num(); i++)
	{
		const FZoneShapeConnector& Connector = Connectors[i];
		const FZoneShapeConnection* Connection = i < Connections.Num() ? &Connections[i] : nullptr;
		DrawZoneShapeConnector(Connector, Connection, PDI, LocalToWorld, DepthBias);
	}
}

void DrawZoneShapeConnector(const FZoneShapeConnector& Connector, const FZoneShapeConnection* Connection, FPrimitiveDrawInterface* PDI, const FMatrix& LocalToWorld, const float DepthBias)
{
	const FColor ConnectedColor(255,255,255);
	const FColor UnconnectedColor(128,128,128);
	const float LineThickness = 2.0f;
	const float ChevronSize = 20.0f;
	const float SmallChevronSize = 12.0f;

	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
	check(ZoneGraphSettings);
	const FZoneGraphTagMask VisTagMask = ZoneGraphSettings->GetVisualizedTags();
	const TConstArrayView<FZoneGraphTagInfo> TagInfo = ZoneGraphSettings->GetTagInfos();

	const bool bIsConnected = Connection != nullptr && Connection->ShapeComponent != nullptr;

    const FColor Color = bIsConnected ? ConnectedColor : UnconnectedColor;

    const FVector WorldPosition = LocalToWorld.TransformPosition(Connector.Position);
    const FVector WorldNormal = LocalToWorld.TransformVector(Connector.Normal);
    const FVector WorldUp = LocalToWorld.TransformVector(Connector.Up);
    const FVector WorldSide = FVector::CrossProduct(WorldNormal, WorldUp);

    // Draw chevron to indicate a connector center.
    PDI->DrawLine(WorldPosition - WorldNormal * ChevronSize, WorldPosition - WorldSide * ChevronSize, Color, SDPG_World, LineThickness, DepthBias, true);
    PDI->DrawLine(WorldPosition - WorldNormal * ChevronSize, WorldPosition + WorldSide * ChevronSize, Color, SDPG_World, LineThickness, DepthBias, true);

    // Draw small chevrons indicating the lane locations and directions. This is useful when debugging non-connect lanes.
    if (const FZoneLaneProfile* LaneProfile = ZoneGraphSettings->GetLaneProfileByRef(Connector.LaneProfile))
    {
        const float TotalWidth = LaneProfile->GetLanesTotalWidth();
        const float HalfWidth = TotalWidth * 0.5f;
        float CurWidth = 0.0f;
        const float ReverseScale = Connector.bReverseLaneProfile ? -1.0f : 1.0f;
        for (const FZoneLaneDesc& LaneDesc : LaneProfile->Lanes)
        {
         	// Skip spacer lanes
         	if (LaneDesc.Direction == EZoneLaneDirection::None)
         	{
         		CurWidth += LaneDesc.Width;
         		continue;
         	}
         	const float LanePos = -HalfWidth + CurWidth + LaneDesc.Width * 0.5f;
         	FVector LaneStartPosition = WorldPosition - WorldSide * LanePos * ReverseScale;
         	const FColor LaneColor = GetTagMaskColor(LaneDesc.Tags & VisTagMask, TagInfo);

         	bool bForward = LaneDesc.Direction == EZoneLaneDirection::Forward;
         	if (Connector.bReverseLaneProfile)
         	{
         		bForward = !bForward;
         	}
         	if (bForward)
         	{
         		PDI->DrawLine(LaneStartPosition - WorldNormal * SmallChevronSize, LaneStartPosition - WorldSide * SmallChevronSize*0.5f, LaneColor, SDPG_World, 1.0, DepthBias, true);
         		PDI->DrawLine(LaneStartPosition - WorldNormal * SmallChevronSize, LaneStartPosition + WorldSide * SmallChevronSize * 0.5f, LaneColor, SDPG_World, 1.0, DepthBias, true);
         	}
         	else
         	{
         		PDI->DrawLine(LaneStartPosition, LaneStartPosition - WorldSide * SmallChevronSize * 0.5f - WorldNormal * SmallChevronSize, LaneColor, SDPG_World, 1.0, DepthBias, true);
         		PDI->DrawLine(LaneStartPosition, LaneStartPosition + WorldSide * SmallChevronSize * 0.5f - WorldNormal * SmallChevronSize, LaneColor, SDPG_World, 1.0, DepthBias, true);
         	}

         	CurWidth += LaneDesc.Width;
        }
    }
}
	
void DrawLaneDirections(const FZoneGraphStorage& ZoneStorage, FPrimitiveDrawInterface* PDI, const FZoneGraphLaneHandle LaneHandle, const FColor Color)
{
	if (!ensure(LaneHandle.IsValid()))
	{
		return;
	}

	const float VectorLength = 150.0f;
	const FVector OffsetZ(0.f, 0.f, 0.1f);
	const FZoneLaneData& Lane = ZoneStorage.Lanes[LaneHandle.Index];
	for (int32 i = Lane.PointsBegin; i < Lane.PointsEnd; i++)
	{
		const FVector Point = ZoneStorage.LanePoints[i];
		const FVector Forward = ZoneStorage.LaneTangentVectors[i];
		PDI->DrawLine(Point + OffsetZ, Point + Forward * VectorLength + OffsetZ, Color, SDPG_World, 2.0f);
	}
}
	
void DrawLaneSmoothing(const FZoneGraphStorage& ZoneStorage, FPrimitiveDrawInterface* PDI, const FZoneGraphLaneHandle LaneHandle, const FColor Color)
{
	if (!ensure(LaneHandle.IsValid()))
	{
		return;
	}

	const FZoneLaneData& Lane = ZoneStorage.Lanes[LaneHandle.Index];

	const FVector OffsetZ(0.f, 0.f, 0.1f);
	const FVector TickOffsetZ(0.f, 0.f, 0.5f);
	const float TickSpacing = Lane.Width * 0.5f;
	
	// Draw smoother lanes using per point forward direction and cubic bezier interpolation.
	FVector PrevPoint = ZoneStorage.LanePoints[Lane.PointsBegin];
	for (int32 i = Lane.PointsBegin; i < Lane.PointsEnd - 1; i++)
	{
		const FVector StartPoint = ZoneStorage.LanePoints[i];
		const FVector StartForward = ZoneStorage.LaneTangentVectors[i];
		const FVector EndPoint = ZoneStorage.LanePoints[i + 1];
		const FVector EndForward = ZoneStorage.LaneTangentVectors[i + 1];
		const float TangentDistance = FVector::Dist(StartPoint, EndPoint) / 3.0f;
		const FVector StartControlPoint = StartPoint + StartForward * TangentDistance;
		const FVector EndControlPoint = EndPoint - EndForward * TangentDistance;

		const float ApproxArcLength = FVector::Dist(StartPoint, StartControlPoint) + FVector::Dist(StartControlPoint, EndControlPoint) + FVector::Dist(EndControlPoint, EndPoint);
		const int32 NumTicks = FMath::Max(1, FMath::CeilToInt(ApproxArcLength / TickSpacing));

		const float DeltaT = 1.0f / NumTicks;
		for (int32 j = 0; j < NumTicks; j++)
		{
			const float T = (j + 1) * DeltaT;
			const FVector Point = UE::CubicBezier::Eval(StartPoint, StartControlPoint, EndControlPoint, EndPoint, T);
			PDI->DrawLine(PrevPoint + OffsetZ, Point + OffsetZ, Color, SDPG_World, 4.0f);
			PDI->DrawPoint(Point + TickOffsetZ, FLinearColor(Color) * 0.5f, 8.0f, SDPG_World);
			PrevPoint = Point;
		}
	}
}

void AppendLane(FDebugRenderSceneProxy* DebugProxy, const FZoneGraphStorage& ZoneStorage, const FZoneGraphLaneHandle& LaneHandle,
				const FColor Color, const float LineThickness /*= 5.f*/, const FVector& Offset /*= FVector::ZeroVector*/)
{
	if (!ensure(LaneHandle.IsValid()))
	{
		return;
	}

	const FVector OffsetZ(0.f, 0.f, 0.1f);
	const FZoneLaneData& Lane = ZoneStorage.Lanes[LaneHandle.Index];
	FVector PrevPoint = ZoneStorage.LanePoints[Lane.PointsBegin] + Offset;
	for (int32 i = Lane.PointsBegin + 1; i < Lane.PointsEnd; i++)
	{
		const FVector Point = ZoneStorage.LanePoints[i] + Offset;
		DebugProxy->Lines.Emplace(PrevPoint + OffsetZ, Point + OffsetZ, Color, LineThickness);
		PrevPoint = Point;
	}
}

void AppendLane(FDebugRenderSceneProxy* DebugProxy, const FZoneGraphStorage& ZoneStorage,
				const FZoneGraphLaneLocation& InStartLocation, const FZoneGraphLaneLocation& InEndLocation,
				const FColor Color, const float LineThickness, const FVector& Offset)
{
	FZoneGraphLaneLocation StartLocation = InStartLocation;
	FZoneGraphLaneLocation EndLocation = InEndLocation;

	// At least one location must be valid
	ensure(StartLocation.LaneHandle.IsValid() || EndLocation.LaneHandle.IsValid());
	const FZoneLaneData& Lane = StartLocation.IsValid() ? ZoneStorage.Lanes[StartLocation.LaneHandle.Index] : ZoneStorage.Lanes[EndLocation.LaneHandle.Index];

	// If both are valid, they must be on the same lane
	if (StartLocation.LaneHandle.IsValid() && EndLocation.LaneHandle.IsValid())
	{
		ensure(StartLocation.LaneHandle.Index == EndLocation.LaneHandle.Index);

		if (InStartLocation.DistanceAlongLane > InEndLocation.DistanceAlongLane)
		{
			// Swap
			StartLocation = InEndLocation;
			EndLocation   = InStartLocation;
		}
	}

	const FVector OffsetZ = Offset + FVector(0.f, 0.f, 0.1f);
	const int32 PointsBegin = StartLocation.LaneHandle.IsValid() ? StartLocation.LaneSegment : Lane.PointsBegin;
	const int32 PointsEnd = EndLocation.LaneHandle.IsValid() ? EndLocation.LaneSegment + 1 : Lane.PointsEnd;

	FVector PrevPoint = StartLocation.LaneHandle.IsValid() ? StartLocation.Position : ZoneStorage.LanePoints[Lane.PointsBegin];
	for (int32 i = PointsBegin + 1; i < PointsEnd; i++)
	{
		const FVector Point = ZoneStorage.LanePoints[i];
		DebugProxy->Lines.Emplace(PrevPoint + OffsetZ, Point + OffsetZ, Color, LineThickness);
		PrevPoint = Point;
	}

	if (EndLocation.LaneHandle.IsValid())
	{
		// Last (partial segment)
		DebugProxy->Lines.Emplace(PrevPoint + OffsetZ, EndLocation.Position + OffsetZ, Color, LineThickness);
	}
}

void AppendLaneSection(FDebugRenderSceneProxy* DebugProxy, const FZoneGraphStorage& ZoneStorage,
	const FZoneGraphLaneSection& Section, const FColor Color, const float LineThickness, const FVector& Offset)
{
	FZoneGraphLaneLocation Start;
	FZoneGraphLaneLocation End;
	if (UE::ZoneGraph::Query::CalculateLocationAlongLane(ZoneStorage, Section.LaneHandle, Section.StartDistanceAlongLane, Start) &&
		UE::ZoneGraph::Query::CalculateLocationAlongLane(ZoneStorage, Section.LaneHandle, Section.EndDistanceAlongLane, End))
	{
		AppendLane(DebugProxy, ZoneStorage, Start, End, Color, LineThickness, Offset);
	}	
}

} // namespace UE::ZoneGraph::RenderingUtilities
