// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZoneGraphSettings.h"
#include "ZoneGraphDelegates.h"
#include "ZoneGraphObjectCRC32.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ZoneGraphSettings)

namespace UE::ZoneGraph
{
// Array of random colors
static FColor Colors[32] = {
	FColor(89, 168, 226),
	FColor(239, 90, 82),
	FColor(243, 70, 114),
	FColor(168, 135, 222),
	FColor(195, 218, 55),
	FColor(229, 196, 44),
	FColor(119, 206, 78),
	FColor(155, 192, 116),
	FColor(215, 142, 197),
	FColor(69, 216, 210),
	FColor(105, 201, 153),
	FColor(102, 204, 106),
	FColor(231, 135, 32),
	FColor(221, 138, 106),
	FColor(110, 141, 233),
	FColor(199, 113, 229),
	FColor(223, 125, 72),
	FColor(153, 177, 71),
	FColor(183, 151, 219),
	FColor(233, 102, 156),
	FColor(202, 158, 48),
	FColor(224, 120, 124),
	FColor(81, 134, 247),
	FColor(234, 86, 198),
	FColor(138, 115, 246),
	FColor(206, 160, 74),
	FColor(218, 123, 204),
	FColor(199, 162, 94),
	FColor(89, 208, 135),
	FColor(221, 226, 134),
	FColor(188, 203, 75),
	FColor(235, 96, 45),
};

FColor GetColor(int32 Index)
{
	constexpr int32 MaxColors = sizeof(Colors) / sizeof(Colors[0]);
	const int32 PaletteIndex = FMath::Abs(Index) % MaxColors;
	return Colors[PaletteIndex];
}

namespace Helpers
{
	FName GetTagName(const FZoneGraphTag Tag)
	{
		const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
		check(ZoneGraphSettings);
		const TConstArrayView<FZoneGraphTagInfo> Tags = ZoneGraphSettings->GetTagInfos();
		for (int32 i = 0; i < static_cast<int32>(EZoneGraphTags::MaxTags); i++)
		{
			const FZoneGraphTagInfo& Info = Tags[i];
			if (Info.IsValid() && Info.Tag == Tag)
			{
				return Info.Name;
			}
		}
		return FName(TEXT("Invalid tag"));
	}

	FString GetTagMaskString(const FZoneGraphTagMask TagMask, const TCHAR* Separator)
	{
		const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
		check(ZoneGraphSettings);

		FString Result;
		const TConstArrayView<FZoneGraphTagInfo> Tags = ZoneGraphSettings->GetTagInfos();
		for (int32 i = 0; i < static_cast<int32>(EZoneGraphTags::MaxTags); i++)
		{
			const FZoneGraphTagInfo& Info = Tags[i];
			if (TagMask.Contains(Info.Tag))
			{
				if (!Result.IsEmpty())
				{
					Result += Separator;
				}
				Result += Info.Name.ToString();
			}
		}

		return Result;
	}
}
} // namespace UE::ZoneGraph

UZoneGraphSettings::UZoneGraphSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Setup default values, the config file will override these when the user changes them.
	for (int32 i = 0; i < static_cast<int32>(EZoneGraphTags::MaxTags); i++)
	{
		FZoneGraphTagInfo& Info = Tags[i];
		Info.Tag = FZoneGraphTag(static_cast<uint8>(i));
		Info.Color = UE::ZoneGraph::GetColor(i);
	}
	Tags[0].Name = FName(TEXT("Default"));
}

//----------------------------------------------------------------------//
// UZoneGraphSettings
//----------------------------------------------------------------------//
const FZoneLaneProfile* UZoneGraphSettings::GetLaneProfileByRef(const FZoneLaneProfileRef& LaneProfileRef) const
{
	return LaneProfiles.FindByPredicate([&LaneProfileRef](const FZoneLaneProfile& LaneProfile) -> bool { return LaneProfile.ID == LaneProfileRef.ID; });
}

const FZoneLaneProfile* UZoneGraphSettings::GetLaneProfileByID(const FGuid& ID) const
{
	return LaneProfiles.FindByPredicate([&ID](const FZoneLaneProfile& LaneProfile) -> bool { return LaneProfile.ID == ID; });
}

#if WITH_EDITOR
void UZoneGraphSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	FProperty* Property = PropertyChangedEvent.Property;
	FProperty* MemberProperty = nullptr;
	if (PropertyChangedEvent.PropertyChain.GetActiveMemberNode())
	{
		MemberProperty = PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue();
	}

	if (MemberProperty && Property)
	{
		if (MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UZoneGraphSettings, LaneProfiles))
		{
			const int32 ArrayIndex = PropertyChangedEvent.GetArrayIndex(MemberProperty->GetFName().ToString());

			// Ensure unique ID on duplicated items.
			if (PropertyChangedEvent.ChangeType == EPropertyChangeType::Duplicate)
			{
				if (LaneProfiles.IsValidIndex(ArrayIndex))
				{
					LaneProfiles[ArrayIndex].ID = FGuid::NewGuid();
					LaneProfiles[ArrayIndex].Name = FName(LaneProfiles[ArrayIndex].Name.ToString() + TEXT(" Duplicate"));
				}
			}

			FZoneLaneProfileRef ChangedLaneProfileRef;
			if (LaneProfiles.IsValidIndex(ArrayIndex))
			{
				ChangedLaneProfileRef = LaneProfiles[ArrayIndex];
			}

			UE::ZoneGraphDelegates::OnZoneGraphLaneProfileChanged.Broadcast(ChangedLaneProfileRef);
		}
		if (MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UZoneGraphSettings, Tags))
		{
			UE::ZoneGraphDelegates::OnZoneGraphTagsChanged.Broadcast();
		}
		if (MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UZoneGraphSettings, BuildSettings))
		{
			UE::ZoneGraphDelegates::OnZoneGraphBuildSettingsChanged.Broadcast();
		}
	}
}

const FZoneLaneProfile* UZoneGraphSettings::GetDefaultLaneProfile() const
{
	return LaneProfiles.Num() > 0 ? &LaneProfiles[0] : nullptr;
}

#endif // WITH_EDITOR

TConstArrayView<FZoneGraphTagInfo> UZoneGraphSettings::GetTagInfos() const
{
	return TConstArrayView<FZoneGraphTagInfo>(Tags, static_cast<int32>(EZoneGraphTags::MaxTags));
}

void UZoneGraphSettings::GetValidTagInfos(TArray<FZoneGraphTagInfo>& OutInfos) const
{
	for (int32 i = 0; i < static_cast<int32>(EZoneGraphTags::MaxTags); i++)
	{
		const FZoneGraphTagInfo& Info = Tags[i];
		if (Info.IsValid())
		{
			OutInfos.Add(Info);
		}
	}
}

#if WITH_EDITOR
uint32 UZoneGraphSettings::GetBuildHash() const
{
	FZoneGraphObjectCRC32 Archive;
	return Archive.Crc32(const_cast<UObject*>((const UObject*)this), 0);
}
#endif
