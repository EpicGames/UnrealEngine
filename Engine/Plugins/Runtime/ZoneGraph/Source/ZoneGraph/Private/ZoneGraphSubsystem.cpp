// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZoneGraphSubsystem.h"
#include "ZoneShapeComponent.h"
#include "ZoneGraphData.h"
#include "ZoneGraphBuilder.h"
#include "ZoneGraphQuery.h"
#include "ZoneGraphDelegates.h"
#include "ZoneGraphSettings.h"
#include "Engine/World.h"
#include "Engine/LevelBounds.h"
#include "Engine/Level.h"
#include "EngineUtils.h"
#include "MassSubsystemBase.h"
#if WITH_EDITOR
#include "LevelEditorViewport.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ZoneGraphSubsystem)

UZoneGraphSubsystem::UZoneGraphSubsystem()
	: bInitialized(false)
{
}

void UZoneGraphSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

#if WITH_EDITOR
	OnActorMovedHandle = GEngine->OnActorMoved().AddUObject(this, &UZoneGraphSubsystem::OnActorMoved);
	OnRequestRebuildHandle = UE::ZoneGraphDelegates::OnZoneGraphRequestRebuild.AddUObject(this, &UZoneGraphSubsystem::OnRequestRebuild);
#endif

	// Note that we're using GetClass() rather than StaticClass() to work as expected for child-classes as well.
	// Child class can always override the traits registered this way.
	UE::Mass::Subsystems::RegisterSubsystemType(Collection, GetClass(), UE::Mass::FSubsystemTypeTraits::Make<UZoneGraphSubsystem>());

	bInitialized = true;
}

void UZoneGraphSubsystem::PostInitialize()
{
	Super::PostInitialize();

	// Register Zone Graph data that we missed before the subsystem got initialized.
	RegisterZoneGraphDataInstances();
}

void UZoneGraphSubsystem::Deinitialize()
{
#if WITH_EDITOR
	GEngine->OnActorMoved().Remove(OnActorMovedHandle);
	OnActorMovedHandle.Reset();
	UE::ZoneGraphDelegates::OnZoneGraphRequestRebuild.Remove(OnRequestRebuildHandle);
	OnRequestRebuildHandle.Reset();
#endif

	bInitialized = false;

	Super::Deinitialize();
}

void UZoneGraphSubsystem::Tick(float DeltaTime)
{
#if WITH_EDITOR
	const UWorld* World = GetWorld();
	if (!World->IsGameWorld())
	{
		if (Builder.NeedsRebuild())
		{
			const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
			check(ZoneGraphSettings);
			if (ZoneGraphSettings->ShouldBuildZoneGraphWhileEditing())
			{
				RebuildGraph();
			}
		}
	}
	else
	{
		// Zone graph is not meant to update during game tick.
		ensureMsgf(!Builder.NeedsRebuild(), TEXT("Builder should not need update during game."));
	}
#endif
}

TStatId UZoneGraphSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UZoneGraphSubsystem, STATGROUP_Tickables);
}

FZoneGraphDataHandle UZoneGraphSubsystem::RegisterZoneGraphData(AZoneGraphData& InZoneGraphData)
{
	if (!IsValid(&InZoneGraphData))
	{
		return FZoneGraphDataHandle();
	}

	if (InZoneGraphData.IsRegistered())
	{
		UE_LOGF(LogZoneGraph, Error, "Trying to register already registered ZoneGraphData \'%ls\'", *InZoneGraphData.GetName());
		return FZoneGraphDataHandle();
	}

	FScopeLock Lock(&DataRegistrationSection);

	if (RegisteredZoneGraphData.FindByPredicate([&InZoneGraphData](const FRegisteredZoneGraphData& Item) { return Item.bInUse && Item.ZoneGraphData == &InZoneGraphData; }) != nullptr)
	{
		UE_LOGF(LogZoneGraph, Error, "ZoneGraphData \'%ls\' already exists in RegisteredZoneGraphData.", *InZoneGraphData.GetName());
		return FZoneGraphDataHandle();
	}

	const int32 Index = (ZoneGraphDataFreeList.Num() > 0) ? ZoneGraphDataFreeList.Pop(EAllowShrinking::No) : RegisteredZoneGraphData.AddDefaulted();
	FRegisteredZoneGraphData& RegisteredData = RegisteredZoneGraphData[Index];
	RegisteredData.Reset(RegisteredData.Generation); // Do not change generation.
	RegisteredData.ZoneGraphData = &InZoneGraphData;
	RegisteredData.bInUse = true;
	check(Index < int32(MAX_uint16));
	const FZoneGraphDataHandle ResultHandle = FZoneGraphDataHandle(uint16(Index), uint16(RegisteredData.Generation));

	InZoneGraphData.OnRegistered(ResultHandle);

	UE::ZoneGraphDelegates::OnPostZoneGraphDataAdded.Broadcast(RegisteredData.ZoneGraphData);

	return ResultHandle;
}

void UZoneGraphSubsystem::UnregisterZoneGraphData(AZoneGraphData& InZoneGraphData)
{
	FScopeLock Lock(&DataRegistrationSection);

	const int32 Index = RegisteredZoneGraphData.IndexOfByPredicate([&InZoneGraphData](const FRegisteredZoneGraphData& Item) { return Item.bInUse && Item.ZoneGraphData == &InZoneGraphData; });
	if (Index == INDEX_NONE)
	{
		UE_LOGF(LogZoneGraph, Error, "Trying to remove ZoneGraphData \'%ls\' that does not exists in RegisteredZoneGraphData.", *InZoneGraphData.GetName());
		return;
	}

	RemoveRegisteredDataItem(Index);

	InZoneGraphData.OnUnregistered();
}

void UZoneGraphSubsystem::RemoveRegisteredDataItem(const int32 Index)
{
	FRegisteredZoneGraphData& RegisteredData = RegisteredZoneGraphData[Index];
	check(int32(RegisteredData.Generation + 1) < int32(MAX_uint16));

	UE::ZoneGraphDelegates::OnPreZoneGraphDataRemoved.Broadcast(RegisteredData.ZoneGraphData);

	RegisteredData.Reset(RegisteredData.Generation + 1);	// Bump generation, so that uses of stale handles can be detected.

	// Mark index to be reused.
	ZoneGraphDataFreeList.Add(Index);
}

void UZoneGraphSubsystem::UnregisterStaleZoneGraphDataInstances()
{
	// Unregister data that have gone stale.
	for (int32 Index = 0; Index < RegisteredZoneGraphData.Num(); Index++)
	{
		if (RegisteredZoneGraphData[Index].bInUse)
		{
			const AZoneGraphData* ZoneGraphData = RegisteredZoneGraphData[Index].ZoneGraphData;
			if (!IsValid(ZoneGraphData))
			{
				UE_LOGF(LogZoneGraph, Error, "Removing stale ZoneGraphData \'%ls\' that does not exists in RegisteredZoneGraphData.", *GetNameSafe(ZoneGraphData));
				RemoveRegisteredDataItem(Index);
				Index--;
			}
			else if (!ZoneGraphData->IsRegistered())
			{
				UE_LOGF(LogZoneGraph, Error, "Removing unregistered ZoneGraphData \'%ls\' that does not exists in RegisteredZoneGraphData.", *GetNameSafe(ZoneGraphData));
				RemoveRegisteredDataItem(Index);
				Index--;
			}
		}
	}
}

void UZoneGraphSubsystem::RegisterZoneGraphDataInstances()
{
	UWorld* World = GetWorld();

	// Make sure all data is registered.
	for (TActorIterator<AZoneGraphData> It(World); It; ++It)
	{
		AZoneGraphData* ZoneGraphData = (*It);
		if (ZoneGraphData != nullptr && IsValidChecked(ZoneGraphData) == true && ZoneGraphData->IsRegistered() == false)
		{
			RegisterZoneGraphData(*ZoneGraphData);
		}
	}
}

#if WITH_EDITOR
void UZoneGraphSubsystem::OnActorMoved(AActor* Actor)
{
	if (Actor != nullptr)
	{
		if (UZoneShapeComponent* ShapeComp = Actor->FindComponentByClass<UZoneShapeComponent>())
		{
			Builder.OnZoneShapeComponentChanged(*ShapeComp);
		}
	}
}

void UZoneGraphSubsystem::OnRequestRebuild()
{
	RebuildGraph(true /*bForceRebuild*/);

	// Force update viewport
	if (GCurrentLevelEditingViewportClient)
	{
		GCurrentLevelEditingViewportClient->Invalidate();
	}
}

void UZoneGraphSubsystem::SpawnMissingZoneGraphData()
{
	// Make sure each level which has zones has its own ZoneGraphData.

	// Find the levels where the splines are located.
	TSet<ULevel*> SupportedLevels;
	for (const FZoneGraphBuilderRegisteredComponent& Registered : Builder.GetRegisteredZoneShapeComponents())
	{
		if (Registered.Component)
		{
			if (ULevel* OwningLevel = Registered.Component->GetComponentLevel())
			{
				SupportedLevels.Add(OwningLevel);
			}
		}
	}

	// Remove worlds which already has data.
	for (const FRegisteredZoneGraphData& RegisteredData : RegisteredZoneGraphData)
	{
		if (RegisteredData.ZoneGraphData)
		{
			if (const ULevel* Level = RegisteredData.ZoneGraphData->GetLevel())
			{
				SupportedLevels.Remove(Level);
			}
		}
	}

	UWorld* World = GetWorld();

	// Create new data for missing worlds.
	for (ULevel* Level : SupportedLevels)
	{
		FActorSpawnParameters SpawnInfo;
		SpawnInfo.OverrideLevel = Level;
		World->SpawnActor<AZoneGraphData>(AZoneGraphData::StaticClass(), SpawnInfo);
	}
}

void UZoneGraphSubsystem::RebuildGraph(const bool bForceRebuild)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UZoneGraphSubsystem::RebuildGraph);

	// Make sure we have zone graph data for each world which has splines.
	UnregisterStaleZoneGraphDataInstances();
	RegisterZoneGraphDataInstances();
	SpawnMissingZoneGraphData();

	TArray<AZoneGraphData*> AllZoneGraphData;
	AllZoneGraphData.Reserve(RegisteredZoneGraphData.Num());

	for (const FRegisteredZoneGraphData& RegisteredData : RegisteredZoneGraphData)
	{
		if (RegisteredData.ZoneGraphData)
		{
			AllZoneGraphData.Add(RegisteredData.ZoneGraphData);
		}
	}

	Builder.BuildAll(AllZoneGraphData, bForceRebuild);
}

#endif // WITH_EDITOR


const AZoneGraphData* UZoneGraphSubsystem::GetZoneGraphData(const FZoneGraphDataHandle DataHandle) const
{
	if (int32(DataHandle.Index) < RegisteredZoneGraphData.Num() && int32(DataHandle.Generation) == RegisteredZoneGraphData[DataHandle.Index].Generation)
	{
		return RegisteredZoneGraphData[DataHandle.Index].ZoneGraphData;
	}
	return nullptr;
}

bool UZoneGraphSubsystem::FindNearestLane(const FBox& QueryBounds, const FZoneGraphTagFilter TagFilter, FZoneGraphLaneLocation& OutLaneLocation, float& OutDistanceSqr) const
{
	bool bResult = false;
	float MinDistanceSqr = QueryBounds.GetExtent().SizeSquared();

	OutLaneLocation.Reset();

	for (const FRegisteredZoneGraphData& RegisteredData : RegisteredZoneGraphData)
	{
		if (RegisteredData.ZoneGraphData)
		{
			const FZoneGraphStorage& Storage = RegisteredData.ZoneGraphData->GetStorage();
			if (QueryBounds.Intersect(Storage.Bounds))
			{
				float DistanceSqr = 0.0f;
				FZoneGraphLaneLocation LaneLocation;
				if (UE::ZoneGraph::Query::FindNearestLane(Storage, QueryBounds, TagFilter, LaneLocation, DistanceSqr))
				{
					if (DistanceSqr < MinDistanceSqr)
					{
						MinDistanceSqr = DistanceSqr;
						OutDistanceSqr = DistanceSqr;
						OutLaneLocation = LaneLocation;
						bResult = true;
					}
				}
			}
		}
	}

	return bResult;
}

bool UZoneGraphSubsystem::FindOverlappingLanes(const FBox& QueryBounds, const FZoneGraphTagFilter TagFilter, TArray<FZoneGraphLaneHandle>& OutLanes) const
{
	bool bResult = false;

	OutLanes.Reset();

	for (const FRegisteredZoneGraphData& RegisteredData : RegisteredZoneGraphData)
	{
		if (RegisteredData.ZoneGraphData)
		{
			const FZoneGraphStorage& Storage = RegisteredData.ZoneGraphData->GetStorage();
			if (QueryBounds.Intersect(Storage.Bounds))
			{
				if (UE::ZoneGraph::Query::FindOverlappingLanes(Storage, QueryBounds, TagFilter, OutLanes))
				{
					bResult = true;
				}
			}
		}
	}

	return bResult;
	
}

bool UZoneGraphSubsystem::FindLaneOverlaps(const FVector& Center, const float Radius, const FZoneGraphTagFilter TagFilter, TArray<FZoneGraphLaneSection>& OutLaneSections) const
{
	bool bResult = false;

	OutLaneSections.Reset();

	for (const FRegisteredZoneGraphData& RegisteredData : RegisteredZoneGraphData)
	{
		if (RegisteredData.ZoneGraphData)
		{
			const FZoneGraphStorage& Storage = RegisteredData.ZoneGraphData->GetStorage();
			FBox Bounds(Center, Center);
			Bounds = Bounds.ExpandBy(Radius);
			if (Bounds.Intersect(Storage.Bounds))
			{
				if (UE::ZoneGraph::Query::FindLaneOverlaps(Storage, Center, Radius, TagFilter, OutLaneSections))
				{
					bResult = true;
				}
			}
		}
	}

	return bResult;
}

bool UZoneGraphSubsystem::AdvanceLaneLocation(const FZoneGraphLaneLocation& InLaneLocation, const float AdvanceDistance, FZoneGraphLaneLocation& OutLaneLocation) const
{
	const AZoneGraphData* Data = GetZoneGraphData(InLaneLocation.LaneHandle.DataHandle);
	return Data && UE::ZoneGraph::Query::AdvanceLaneLocation(Data->GetStorage(), InLaneLocation, AdvanceDistance, OutLaneLocation);
}

bool UZoneGraphSubsystem::CalculateLocationAlongLane(const FZoneGraphLaneHandle LaneHandle, const float Distance, FZoneGraphLaneLocation& OutLaneLocation) const
{
	const AZoneGraphData* Data = GetZoneGraphData(LaneHandle.DataHandle);
	return Data && UE::ZoneGraph::Query::CalculateLocationAlongLane(Data->GetStorage(), LaneHandle, Distance, OutLaneLocation);
}

bool UZoneGraphSubsystem::FindNearestLocationOnLane(const FZoneGraphLaneHandle LaneHandle, const FBox& Bounds, FZoneGraphLaneLocation& OutLaneLocation, float& OutDistanceSqr) const
{
	const AZoneGraphData* Data = GetZoneGraphData(LaneHandle.DataHandle);
	return Data && UE::ZoneGraph::Query::FindNearestLocationOnLane(Data->GetStorage(), LaneHandle, Bounds, OutLaneLocation, OutDistanceSqr);
}

bool UZoneGraphSubsystem::FindNearestLocationOnLane(const FZoneGraphLaneHandle LaneHandle, const FVector& Center, const float Range, FZoneGraphLaneLocation& OutLaneLocation, float& OutDistanceSqr) const
{
	const AZoneGraphData* Data = GetZoneGraphData(LaneHandle.DataHandle);
	return Data && UE::ZoneGraph::Query::FindNearestLocationOnLane(Data->GetStorage(), LaneHandle, Center, Range, OutLaneLocation, OutDistanceSqr);
}

bool UZoneGraphSubsystem::IsLaneValid(const FZoneGraphLaneHandle LaneHandle) const
{
	const AZoneGraphData* Data = GetZoneGraphData(LaneHandle.DataHandle);
	return Data && LaneHandle.Index < Data->GetStorage().Lanes.Num();
}

bool UZoneGraphSubsystem::GetLaneLength(const FZoneGraphLaneHandle LaneHandle, float& OutLength) const
{
	const AZoneGraphData* Data = GetZoneGraphData(LaneHandle.DataHandle);
	return Data && UE::ZoneGraph::Query::GetLaneLength(Data->GetStorage(), LaneHandle, OutLength);
}

bool UZoneGraphSubsystem::GetLaneWidth(const FZoneGraphLaneHandle LaneHandle, float& OutWidth) const
{
	const AZoneGraphData* Data = GetZoneGraphData(LaneHandle.DataHandle);
	return Data && UE::ZoneGraph::Query::GetLaneWidth(Data->GetStorage(), LaneHandle, OutWidth);
}

bool UZoneGraphSubsystem::GetLaneTags(const FZoneGraphLaneHandle LaneHandle, FZoneGraphTagMask& OutTags) const
{
	const AZoneGraphData* Data = GetZoneGraphData(LaneHandle.DataHandle);
	return Data && UE::ZoneGraph::Query::GetLaneTags(Data->GetStorage(), LaneHandle, OutTags);
}

bool UZoneGraphSubsystem::GetLinkedLanes(const FZoneGraphLaneHandle LaneHandle, const EZoneLaneLinkType Types, const EZoneLaneLinkFlags IncludeFlags, const EZoneLaneLinkFlags ExcludeFlags, TArray<FZoneGraphLinkedLane>& OutLinkedLanes) const
{
	const AZoneGraphData* Data = GetZoneGraphData(LaneHandle.DataHandle);
	return Data && UE::ZoneGraph::Query::GetLinkedLanes(Data->GetStorage(), LaneHandle, Types, IncludeFlags, ExcludeFlags, OutLinkedLanes);
}

bool UZoneGraphSubsystem::GetFirstLinkedLane(const FZoneGraphLaneHandle LaneHandle, const EZoneLaneLinkType Types, const EZoneLaneLinkFlags IncludeFlags, const EZoneLaneLinkFlags ExcludeFlags, FZoneGraphLinkedLane& OutLinkedLane) const
{
	const AZoneGraphData* Data = GetZoneGraphData(LaneHandle.DataHandle);
	return Data && UE::ZoneGraph::Query::GetFirstLinkedLane(Data->GetStorage(), LaneHandle, Types, IncludeFlags, ExcludeFlags, OutLinkedLane);
}

FZoneGraphTag UZoneGraphSubsystem::GetTagByName(FName TagName) const
{
	if (const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>())
	{
		const TConstArrayView<FZoneGraphTagInfo> Infos = ZoneGraphSettings->GetTagInfos();
		for (const FZoneGraphTagInfo& Info : Infos)
		{
			if (Info.Name == TagName)
			{
				return Info.Tag;
			}
		}
	}
	return FZoneGraphTag::None;
}

FBox UZoneGraphSubsystem::GetCombinedBounds() const
{
	FBox CombinedBounds(ForceInit);
	
	for (const FRegisteredZoneGraphData& RegisteredData : RegisteredZoneGraphData)
	{
		if (RegisteredData.ZoneGraphData)
		{
			const FZoneGraphStorage& Storage = RegisteredData.ZoneGraphData->GetStorage();
			CombinedBounds += Storage.Bounds;
		}
	}

	return CombinedBounds;
}

FName UZoneGraphSubsystem::GetTagName(FZoneGraphTag Tag) const
{
	if (const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>())
	{
		const TConstArrayView<FZoneGraphTagInfo> Infos = ZoneGraphSettings->GetTagInfos();
		if (int32(Tag.Get()) < Infos.Num())
		{
			return Infos[Tag.Get()].Name;
		}
	}
	return FName();
}

const FZoneGraphTagInfo* UZoneGraphSubsystem::GetTagInfo(FZoneGraphTag Tag) const
{
	if (const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>())
	{
		const TConstArrayView<FZoneGraphTagInfo> Infos = ZoneGraphSettings->GetTagInfos();
		if (int32(Tag.Get()) < Infos.Num())
		{
			return &Infos[Tag.Get()];
		}
	}
	return nullptr;
}

TConstArrayView<FZoneGraphTagInfo> UZoneGraphSubsystem::GetTagInfos() const
{
	if (const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>())
	{
		return ZoneGraphSettings->GetTagInfos();
	}
	return TConstArrayView<FZoneGraphTagInfo>();
}
