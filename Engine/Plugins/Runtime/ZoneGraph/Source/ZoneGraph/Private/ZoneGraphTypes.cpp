// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZoneGraphTypes.h"
#include "Algo/Reverse.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ZoneGraphTypes)

DEFINE_LOG_CATEGORY(LogZoneGraph);

const FZoneHandle FZoneHandle::Invalid = FZoneHandle();
const uint8 FZoneShapePoint::InheritLaneProfile = 0xff;
const uint16 FZoneGraphDataHandle::InvalidGeneration = 0;
const uint8 FZoneGraphTag::NoneValue = 0xff;
const FZoneGraphTag FZoneGraphTag::None = FZoneGraphTag();
const FZoneGraphTagMask FZoneGraphTagMask::All = FZoneGraphTagMask(0xffffffff);
const FZoneGraphTagMask FZoneGraphTagMask::None = FZoneGraphTagMask(0);

bool FZoneLaneDesc::operator==(const FZoneLaneDesc& Other) const
{
	return Width == Other.Width && Direction == Other.Direction && Tags == Other.Tags;
}

FVector FZoneShapePoint::GetInControlPoint() const
{
	return Position + Rotation.RotateVector(FVector::BackwardVector) * TangentLength;
}


FVector FZoneShapePoint::GetOutControlPoint() const
{
	return Position + Rotation.RotateVector(FVector::ForwardVector) * TangentLength;
}

void FZoneShapePoint::SetInControlPoint(const FVector& InPoint)
{
	const FVector Tangent = Position - InPoint;
	const FRotator NewRotation = Tangent.Rotation();
	Rotation.Pitch = NewRotation.Pitch;
	Rotation.Yaw = NewRotation.Yaw;
	TangentLength = Tangent.Size();
}

void FZoneShapePoint::SetOutControlPoint(const FVector& InPoint)
{
	const FVector Tangent = InPoint - Position;
	const FRotator NewRotation = Tangent.Rotation();
	Rotation.Pitch = NewRotation.Pitch;
	Rotation.Yaw = NewRotation.Yaw;
	TangentLength = Tangent.Size();
}

FVector FZoneShapePoint::GetLaneProfileLeft() const
{
	return Position + Rotation.RotateVector(FVector::LeftVector) * TangentLength;
}

FVector FZoneShapePoint::GetLaneProfileRight() const
{
	return Position + Rotation.RotateVector(FVector::RightVector) * TangentLength;
}

void FZoneShapePoint::SetLaneProfileLeft(const FVector& InPoint)
{
	const FVector Tangent = Position - InPoint;
	const FRotator NewRotation = Tangent.Rotation();
	Rotation.Pitch = NewRotation.Pitch;
	Rotation.Yaw = NewRotation.Yaw + 90.0f; // For Lane Profile points, the rotation points into the shape.
	TangentLength = Tangent.Size();
}

void FZoneShapePoint::SetLaneProfileRight(const FVector& InPoint)
{
	const FVector Tangent = InPoint - Position;
	const FRotator NewRotation = Tangent.Rotation();
	Rotation.Pitch = NewRotation.Pitch;
	Rotation.Yaw = NewRotation.Yaw + 90.0f;	// For Lane Profile points, the rotation points into the shape.
	TangentLength = Tangent.Size();
}

void FZoneShapePoint::SetRotationFromForwardAndUp(const FVector& Forward, const FVector& Up)
{
	// This creates Look-at rotation towards the Forward direction using pitch/Yaw.
	Rotation = Forward.Rotation();

	// Figure out Roll from Up vector and current rotation.
	const FVector LocalUp = Rotation.Quaternion().UnrotateVector(Up);
	Rotation.Roll = FMath::RadiansToDegrees(FMath::Atan2(LocalUp.Y, LocalUp.Z));
}


void FZoneGraphStorage::Reset()
{
	Zones.Reset();
	Lanes.Reset();
	BoundaryPoints.Reset();
	LanePoints.Reset();
	LaneUpVectors.Reset();
	LaneTangentVectors.Reset();
	LanePointProgressions.Reset();
	LaneLinks.Reset();
	Bounds.Init();
}

bool FZoneLaneProfile::IsSymmetrical() const
{
	const int32 NumLanes = Lanes.Num();
	int32 First = 0;
	int32 Last = NumLanes - 1;
	bool bIsSymmetrical = true;
	while (First < Last)
	{
		const FZoneLaneDesc& FirstLane = Lanes[First];
		const FZoneLaneDesc& LastLane = Lanes[Last];

		// Check if the lanes are compatible when reversed.
		const bool bAreReverseCompatible = FMath::IsNearlyEqual(FirstLane.Width, LastLane.Width)
			&& FirstLane.Tags == LastLane.Tags
			&& FirstLane.Direction != LastLane.Direction;

		if (!bAreReverseCompatible)
		{
			bIsSymmetrical = false;
			break;
		}
		First++;
		Last--;
	}
	return bIsSymmetrical;
}

void FZoneLaneProfile::ReverseLanes()
{
	Algo::Reverse(Lanes);
	for (FZoneLaneDesc& Lane : Lanes)
	{
		Lane.Direction = Lane.Direction == EZoneLaneDirection::Forward ? EZoneLaneDirection::Backward : EZoneLaneDirection::Forward;
	}
}

float FZoneGraphBuildSettings::GetLaneTessellationTolerance(const FZoneGraphTagMask LaneTags) const
{
	float Tolerance = CommonTessellationTolerance;

	for (const FZoneGraphTessellationSettings& Settings : SpecificTessellationTolerances)
	{
		if (Settings.LaneFilter.Pass(LaneTags))
		{
			Tolerance = Settings.TessellationTolerance;
			break;
		}
	}
	return Tolerance;
}

EZoneShapeLaneConnectionRestrictions FZoneGraphBuildSettings::GetConnectionRestrictions(const FZoneGraphTagMask ZoneTags,
																						const FZoneLaneProfileRef& SourceLaneProfile, const int32 SourceOutgoingConnection,
																						const FZoneLaneProfileRef& DestinationLaneProfile, const int32 DestinationIncomingConnection) const
{
	EZoneShapeLaneConnectionRestrictions Result = EZoneShapeLaneConnectionRestrictions::None;

	auto DoesEntryCountMatch = [](const EZoneGraphLaneRoutingCountRule Rule, const int32 Count) -> bool
	{
		switch (Rule)
		{
		case EZoneGraphLaneRoutingCountRule::Any:
			return true;
		case EZoneGraphLaneRoutingCountRule::One:
			return Count == 1;
		case EZoneGraphLaneRoutingCountRule::Many:
			return Count > 1;
		default:
			checkf(false, TEXT("Unhandled case %s"), *StaticEnum<EZoneGraphLaneRoutingCountRule>()->GetNameStringByValue(int64(Rule)));
		}
		return false;
	};
	
	for (const FZoneGraphLaneRoutingRule& Rule : PolygonRoutingRules)
	{
		if (Rule.bEnabled
			&& Rule.ZoneTagFilter.Pass(ZoneTags)
			&& Rule.SourceLaneProfile == SourceLaneProfile
			&& Rule.DestinationLaneProfile == DestinationLaneProfile
			&& DoesEntryCountMatch(Rule.SourceOutgoingConnections, SourceOutgoingConnection)
			&& DoesEntryCountMatch(Rule.DestinationIncomingConnections, DestinationIncomingConnection))
		{
			Result |= EZoneShapeLaneConnectionRestrictions(Rule.ConnectionRestrictions);
		}
	}
	
	return Result;
}
