// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZoneShapeActor.h"
#include "UObject/ConstructorHelpers.h"
#include "ZoneShapeComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ZoneShapeActor)

AZoneShape::AZoneShape(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ShapeComponent = CreateDefaultSubobject<UZoneShapeComponent>(TEXT("ShapeComp"));

	RootComponent = ShapeComponent;

	SetHidden(true);
	SetCanBeDamaged(false);

#if WITH_EDITORONLY_DATA
	bIsSpatiallyLoaded = false;
#endif
}
