// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZoneShapeComponent.h"
#include "Engine/World.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "SceneView.h"
#include "ZoneGraphSubsystem.h"
#include "ZoneGraphSettings.h"
#include "ZoneGraphDelegates.h"
#include "ZoneGraphRenderingUtilities.h"
#include "Curves/BezierUtilities.h"
#include "ZoneGraphObjectCRC32.h"
#include "Serialization/CustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ZoneShapeComponent)


const FGuid FZoneShapeCustomVersion::GUID(0xA6FC0560, 0x45B83348, 0xD0E5E18A, 0x3129CD45);

FCustomVersionRegistration GRegisterZoneShapeComponentCustomVersion(FZoneShapeCustomVersion::GUID, FZoneShapeCustomVersion::LatestVersion, TEXT("ZoneShapeComponent"));


UZoneShapeComponent::UZoneShapeComponent(const FObjectInitializer& ObjectInitializer)
	: UPrimitiveComponent(ObjectInitializer)
	, ShapeType(FZoneShapeType::Spline)
{
	SetCollisionEnabled(ECollisionEnabled::NoCollision);

#if WITH_EDITORONLY_DATA
	HitProxyPriority = HPP_Wireframe;
#endif

	// Add default Shape points
	Points.Emplace(FVector(-400, 0, 0));
	Points.Emplace(FVector(400, 0, 0));

#if WITH_EDITOR
	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
	if (ZoneGraphSettings)
	{
		if (const FZoneLaneProfile* NewLaneProfile = ZoneGraphSettings->GetDefaultLaneProfile())
		{
			LaneProfile = *NewLaneProfile;
		}
	}
#endif
}

void UZoneShapeComponent::OnRegister()
{
	Super::OnRegister();

#if WITH_EDITOR
	// Zone shape is editor only, make sure we dont register them in PIE either.
	const UWorld* World = GetWorld();
	if (!World->IsGameWorld() && HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		if (UZoneGraphSubsystem* ZoneGraph = UWorld::GetSubsystem<UZoneGraphSubsystem>(GetWorld()))
		{
			ZoneGraph->GetBuilder().RegisterZoneShapeComponent(*this);
		}
	}

	OnLaneProfileChangedHandle = UE::ZoneGraphDelegates::OnZoneGraphLaneProfileChanged.AddUObject(this, &UZoneShapeComponent::OnLaneProfileChanged);

	// Update matching shape connections are shapes get registered.
	// This call will also update any connected shapes, so as shapes get registered everything should get update.
	UpdateMatingConnectedShapes();
	
#endif // WITH_EDITOR
}

void UZoneShapeComponent::OnUnregister()
{
	Super::OnUnregister();

#if WITH_EDITOR
	// Zone shape is editor only, make sure we dont register them in PIE either.
	const UWorld* World = GetWorld();
	if (!World->IsGameWorld() && HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		if (UZoneGraphSubsystem* ZoneGraph = UWorld::GetSubsystem<UZoneGraphSubsystem>(GetWorld()))
		{
			ZoneGraph->GetBuilder().UnregisterZoneShapeComponent(*this);
		}
	}

	UE::ZoneGraphDelegates::OnZoneGraphLaneProfileChanged.Remove(OnLaneProfileChangedHandle);
#endif // WITH_EDITOR
}

void UZoneShapeComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FZoneShapeCustomVersion::GUID);
}

void UZoneShapeComponent::PostLoad()
{
	Super::PostLoad();

	const int32 CurrentVersion = GetLinkerCustomVersion(FZoneShapeCustomVersion::GUID);

	if (CurrentVersion < FZoneShapeCustomVersion::AddedRoll)
	{
		// Update rotators from tangents
		const int32 NumPoints = Points.Num();
		for (int32 PointIndex = 0; PointIndex < NumPoints; PointIndex++)
		{
			FZoneShapePoint& Point = Points[PointIndex];
			if (Point.Type == FZoneShapePointType::Sharp)
			{
				int32 PrevPointIndex = 0, NextPointIndex = 0;
				if (IsShapeClosed())
				{
					PrevPointIndex = (PointIndex + NumPoints - 1) % NumPoints;
					NextPointIndex = (PointIndex + 1) % NumPoints;
				}
				else
				{
					PrevPointIndex = PointIndex - 1;
					NextPointIndex = PointIndex + 1;
				}
				const FVector PrevPosition = PrevPointIndex >= 0 ? Points[PrevPointIndex].OutControlPoint_DEPRECATED : Point.Position;
				const FVector NextPosition = NextPointIndex < NumPoints ? Points[NextPointIndex].InControlPoint_DEPRECATED : Point.Position;
				const FVector Tangent = NextPosition - PrevPosition;
				Point.Rotation = Tangent.Rotation();
			}
			else if (Point.Type == FZoneShapePointType::Bezier || Point.Type == FZoneShapePointType::AutoBezier)
			{
				Point.SetInControlPoint(Point.InControlPoint_DEPRECATED);
			}
			else if (Point.Type == FZoneShapePointType::LaneProfile)
			{
				Point.SetLaneProfileLeft(Point.InControlPoint_DEPRECATED);
			}
		}
	}

	// Check that the polygon shape tangents still represent the profiles, if not update the shape.
	if (ShapeType == FZoneShapeType::Polygon)
	{
		bool bNeedsUpdate = false;
		TArray<FZoneLaneProfile> LaneProfiles;
		GetPolygonLaneProfiles(LaneProfiles);

		const int32 NumPoints = GetNumPoints();
		for (int32 PointIndex = 0; PointIndex < NumPoints; PointIndex++)
		{
			const FZoneShapePoint& Point = Points[PointIndex];
			if (Point.Type == FZoneShapePointType::LaneProfile)
			{
				const float ExpectedWidth = LaneProfiles[PointIndex].GetLanesTotalWidth() * 0.5f;
				if (!FMath::IsNearlyEqual(Point.TangentLength, ExpectedWidth))
				{
					bNeedsUpdate = true;
					break;
				}
			}
		}
		if (bNeedsUpdate)
		{
			UpdateShape();
		}
	}

	UpdateShapeConnectors();
}

FBoxSphereBounds UZoneShapeComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FBox BoundingBox(EForceInit::ForceInit);

	const int32 NumPoints = GetNumPoints();
	if (NumPoints > 1)
	{
		int StartIdx = IsShapeClosed() ? (NumPoints - 1) : 0;
		int Idx = IsShapeClosed() ? 0 : 1;

		FVector StartPoint = LocalToWorld.TransformPosition(Points[StartIdx].Position);

		while (Idx < NumPoints)
		{
			const FVector StartControlPoint = LocalToWorld.TransformPosition(Points[StartIdx].GetOutControlPoint());
			const FVector EndControlPoint = LocalToWorld.TransformPosition(Points[Idx].GetInControlPoint());
			const FVector EndPoint = LocalToWorld.TransformPosition(Points[Idx].Position);

			const FBox SegmentBounds = UE::CubicBezier::CalcBounds(StartPoint, StartControlPoint, EndControlPoint, EndPoint);
			BoundingBox += SegmentBounds;

			StartPoint = EndPoint;
			StartIdx = Idx;
			Idx++;
		}

		// Expand bounds from curves by max lane radius, the created bounds should contain all data but might be too big.
		float ConservativeRadius = 0.0f;
		if (ShapeType == FZoneShapeType::Spline)
		{
			FZoneLaneProfile SplineLaneProfile;
			GetSplineLaneProfile(SplineLaneProfile);
			ConservativeRadius = SplineLaneProfile.GetLanesTotalWidth() * 0.5f;
		}
		else if (ShapeType == FZoneShapeType::Polygon)
		{
			TArray<FZoneLaneProfile> PolyLaneProfiles;
			GetPolygonLaneProfiles(PolyLaneProfiles);

			for (const FZoneLaneProfile& Profile : PolyLaneProfiles)
			{
				ConservativeRadius = FMath::Max(ConservativeRadius, Profile.GetLanesTotalWidth() * 0.5f);
			}
		}
		else
		{
			ensureMsgf(false, TEXT("Missing calc bounds for shape type %d"), (int32)(ShapeType));
		}
		BoundingBox = BoundingBox.ExpandBy(ConservativeRadius);
	}

	return BoundingBox;
}

void UZoneShapeComponent::UpdateShape()
{
	const int32 NumPoints = Points.Num();

	bool bUpdatePerPointLaneProfiles = false;

	if (ShapeType == FZoneShapeType::Polygon)
	{
		// Update lane profile points widths to match lane templates.
		TArray<FZoneLaneProfile> LaneProfiles;
		GetPolygonLaneProfiles(LaneProfiles);

		for (int32 PointIndex = 0; PointIndex < NumPoints; PointIndex++)
		{
			FZoneShapePoint& Point = Points[PointIndex];
			if (Point.Type == FZoneShapePointType::LaneProfile)
			{
				Point.TangentLength = LaneProfiles[PointIndex].GetLanesTotalWidth() * 0.5f;
			}
			else
			{
				// Reset lane profile on non-profile points.
				if (Points[PointIndex].LaneProfile != FZoneShapePoint::InheritLaneProfile)
				{
					Points[PointIndex].LaneProfile = FZoneShapePoint::InheritLaneProfile;
					bUpdatePerPointLaneProfiles = true;
				}
			}
		}
	}

	if (bUpdatePerPointLaneProfiles)
	{
		CompactPerPointLaneProfiles();
	}

	// Update rotations and automatic tangents.
	for (int32 i = 0; i < NumPoints; i++)
	{
		if (Points[i].Type == FZoneShapePointType::AutoBezier)
		{
			UpdatePointRotationAndTangent(i);
		}
	}
	// Sharp point orientation depends on auto beziers, update separately.
	for (int32 i = 0; i < NumPoints; i++)
	{
		if (Points[i].Type == FZoneShapePointType::Sharp)
		{
			UpdatePointRotationAndTangent(i);
		}
	}

	// Update connectors
	UpdateMatingConnectedShapes();
	MarkRenderStateDirty();
}

void  UZoneShapeComponent::UpdateMatingConnectedShapes()
{
	// Store which shapes were previously connected to, refresh their potentially mutual connections later.
	TSet<UZoneShapeComponent*> AffectedComponents;
	for (const FZoneShapeConnection& Conn : ConnectedShapes)
	{
		AffectedComponents.Add(Conn.ShapeComponent.Get());
	}

	// Update connectors and find connections.
	UpdateShapeConnectors();
	UpdateConnectedShapes();

	// Store which shapes were got connected to, refresh their potentially mutual connections.
	for (const FZoneShapeConnection& Conn : ConnectedShapes)
	{
		AffectedComponents.Add(Conn.ShapeComponent.Get());
	}

	// Connection may alter the shape, request previous and current connections to update their connections visuals too.
	for (UZoneShapeComponent* AffectedComponent : AffectedComponents)
	{
		if (AffectedComponent)
		{
			AffectedComponent->UpdateConnectedShapes();
			AffectedComponent->MarkRenderStateDirty();
		}
	}
}

void UZoneShapeComponent::UpdateShapeConnectors()
{
	const int32 NumPoints = Points.Num();

	ShapeConnectors.Reset();

	if (ShapeType == FZoneShapeType::Spline)
	{
		// For splines, create a connector at each spline extremity.
		FZoneLaneProfile SplineLaneProfile;
		GetSplineLaneProfile(SplineLaneProfile);

		if (NumPoints > 1)
		{
			FZoneShapeConnector& StartConnector = ShapeConnectors.AddDefaulted_GetRef();
			StartConnector.Position = Points[0].Position;
			StartConnector.Normal = Points[0].Rotation.RotateVector(FVector::BackwardVector);
			StartConnector.Up = Points[0].Rotation.RotateVector(FVector::UpVector);
			StartConnector.PointIndex = 0;
			StartConnector.ShapeType = ShapeType;
			StartConnector.LaneProfile = SplineLaneProfile;
			StartConnector.bReverseLaneProfile = bReverseLaneProfile;

			FZoneShapeConnector& EndConnector = ShapeConnectors.AddDefaulted_GetRef();
			EndConnector.Position = Points[NumPoints - 1].Position;
			EndConnector.Normal = Points[NumPoints - 1].Rotation.RotateVector(FVector::ForwardVector);
			EndConnector.Up = Points[NumPoints - 1].Rotation.RotateVector(FVector::UpVector);
			EndConnector.PointIndex = NumPoints - 1;
			EndConnector.ShapeType = ShapeType;
			EndConnector.LaneProfile = SplineLaneProfile;
			EndConnector.bReverseLaneProfile = !bReverseLaneProfile;	// End connector is pointing different direction than the start connector, the profile needs reversing.
		}
	}
	else if (ShapeType == FZoneShapeType::Polygon)
	{
		// For polygons, create a connect at each lane segment point.
		TArray<FZoneLaneProfile> PolyLaneProfiles;
		GetPolygonLaneProfiles(PolyLaneProfiles);

		for (int32 i = 0; i < NumPoints; i++)
		{
			const FZoneShapePoint& Point = Points[i];
			if (Point.Type == FZoneShapePointType::LaneProfile)
			{
				FZoneShapeConnector& Connector = ShapeConnectors.AddDefaulted_GetRef();
				Connector.Position = Point.Position;
				Connector.Normal = Point.Rotation.RotateVector(FVector::BackwardVector); // Connectors point away from the shape, lane profiles in.
				Connector.Up = Point.Rotation.RotateVector(FVector::UpVector);
				Connector.PointIndex = i;
				Connector.ShapeType = ShapeType;
				Connector.LaneProfile = PolyLaneProfiles[i];
				Connector.bReverseLaneProfile = Point.bReverseLaneProfile;
				if (Point.LaneProfile == FZoneShapePoint::InheritLaneProfile)
				{
					Connector.bReverseLaneProfile = bReverseLaneProfile ? !Connector.bReverseLaneProfile : Connector.bReverseLaneProfile;
				}
			}
		}
	}
	else
	{
		ensureMsgf(false, TEXT("Missing connector update for shape type %d"), (int32)(ShapeType));
	}
}

void UZoneShapeComponent::UpdateConnectedShapes()
{
	// Update connections
	ConnectedShapes.Reset();

#if WITH_EDITOR
	if (UZoneGraphSubsystem* ZoneGraph = UWorld::GetSubsystem<UZoneGraphSubsystem>(GetWorld()))
	{
		ZoneGraph->GetBuilder().FindShapeConnections(*this, ConnectedShapes);
	}
#endif
}

void UZoneShapeComponent::UpdatePointRotationAndTangent(int32 PointIndex)
{
	const int NumPoints = Points.Num();
	if (PointIndex < 0 || PointIndex >= NumPoints )
	{
		return;
	}

	if (NumPoints < 2)
	{
		FZoneShapePoint& Point = Points[PointIndex];
		Point.Rotation = FRotator::ZeroRotator;
		Point.TangentLength = 0.0f;
		return;
	}

	int32 PrevPointIndex = 0, NextPointIndex = 0;
	if (IsShapeClosed())
	{
		PrevPointIndex = (PointIndex + NumPoints - 1) % NumPoints;
		NextPointIndex = (PointIndex + 1) % NumPoints;
	}
	else
	{
		PrevPointIndex = PointIndex - 1;
		NextPointIndex = PointIndex + 1;
	}

	FZoneShapePoint& Point = Points[PointIndex];
	const bool bAllowAutoBezier = Point.Type == FZoneShapePointType::Sharp; // Allow auto bezier handles to affect sharp point's rotation.

	FVector PrevPosition = Point.Position;
	if (PrevPointIndex >= 0)
	{
		const FZoneShapePoint& PrevPoint = Points[PrevPointIndex];
		if (PrevPoint.Type == FZoneShapePointType::Bezier || (bAllowAutoBezier && PrevPoint.Type == FZoneShapePointType::AutoBezier))
		{
			PrevPosition = PrevPoint.GetOutControlPoint();
		}
		else if (PrevPoint.Type == FZoneShapePointType::LaneProfile)
		{
			PrevPosition = PrevPoint.GetLaneProfileRight();
		}
		else
		{
			PrevPosition = PrevPoint.Position;
		}
	}

	FVector NextPosition = Point.Position;
	if (NextPointIndex < NumPoints)
	{
		const FZoneShapePoint& NextPoint = Points[NextPointIndex];
		if (NextPoint.Type == FZoneShapePointType::Bezier || (bAllowAutoBezier && NextPoint.Type == FZoneShapePointType::AutoBezier))
		{
			NextPosition = NextPoint.GetInControlPoint();
		}
		else if (NextPoint.Type == FZoneShapePointType::LaneProfile)
		{
			NextPosition = NextPoint.GetLaneProfileLeft();
		}
		else
		{
			NextPosition = NextPoint.Position;
		}
	}

	const FVector Tangent = (NextPosition - PrevPosition) * 0.5f / 3.0f; // Divide by three for Bezier basis.

	// Update rotation (Roll is controlled manually).
	const FRotator NewRotation = Tangent.Rotation();
	Point.Rotation.Pitch = NewRotation.Pitch;
	Point.Rotation.Yaw = NewRotation.Yaw;

	if (Point.Type == FZoneShapePointType::AutoBezier || Point.Type == FZoneShapePointType::Bezier)
	{
		Point.TangentLength = Tangent.Size();
	}
	else
	{
		Point.TangentLength = 0.0f;
	}
}

bool UZoneShapeComponent::IsShapeClosed() const
{
	if (ShapeType == FZoneShapeType::Spline)
	{
		return false;
	}

	// FZoneShapeType::Polygon
	return true;
}

int32 UZoneShapeComponent::AddUniquePerPointLaneProfile(const FZoneLaneProfileRef& NewLaneProfileRef)
{
	int32 TemplateIndex = PerPointLaneProfiles.IndexOfByPredicate([NewLaneProfileRef](const FZoneLaneProfileRef& LaneProfileRef) -> bool { return NewLaneProfileRef.ID == LaneProfileRef.ID; });
	if (TemplateIndex == INDEX_NONE)
	{
		if (ensure(PerPointLaneProfiles.Num() < (int32)FZoneShapePoint::InheritLaneProfile))
		{
			TemplateIndex = PerPointLaneProfiles.Add(NewLaneProfileRef);
		}
		else
		{
			return INDEX_NONE;
		}
	}

	return TemplateIndex;
}

void UZoneShapeComponent::CompactPerPointLaneProfiles()
{
	if (ensure(ShapeType == FZoneShapeType::Polygon))
	{
		// Find used templates
		TArray<uint8> UsedTemplates;
		for (FZoneShapePoint& Point : Points)
		{
			if (Point.Type == FZoneShapePointType::LaneProfile)
			{
				if (Point.LaneProfile != FZoneShapePoint::InheritLaneProfile)
				{
					UsedTemplates.AddUnique(Point.LaneProfile);
				}
			}
		}

		if (UsedTemplates.Num() != PerPointLaneProfiles.Num())
		{
			if (UsedTemplates.Num() == 0)
			{
				// None used
				PerPointLaneProfiles.Reset();
			}
			else
			{
				// Assumes FZoneLaneProfileRef is simple data, and that PerPointLaneProfiles is rather small array.
				TArray<FZoneLaneProfileRef> OldPerPointLaneProfiles = PerPointLaneProfiles;
				PerPointLaneProfiles.Reset();

				TArray<uint8> Remap;
				Remap.SetNumZeroed(OldPerPointLaneProfiles.Num());

				// Compact array
				for (int32 i = 0; i < UsedTemplates.Num(); i++)
				{
					Remap[UsedTemplates[i]] = (uint8)i;
					PerPointLaneProfiles.Add(OldPerPointLaneProfiles[UsedTemplates[i]]);
				}

				// Remap indices
				for (FZoneShapePoint& Point : Points)
				{
					if (Point.Type == FZoneShapePointType::LaneProfile)
					{
						if (Point.LaneProfile != FZoneShapePoint::InheritLaneProfile)
						{
							Point.LaneProfile = Remap[Point.LaneProfile];
						}
					}
				}

			}
		}
	}
}

void UZoneShapeComponent::ClearPerPointLaneProfiles()
{
	PerPointLaneProfiles.Reset();

	// Make sure there are no references to the array.
	for (FZoneShapePoint& Point : Points)
	{
		if (Point.Type == FZoneShapePointType::LaneProfile)
		{
			Point.LaneProfile = FZoneShapePoint::InheritLaneProfile;
		}
	}
}

const FZoneShapeConnector* UZoneShapeComponent::GetShapeConnectorByPointIndex(int32 PointIndex) const
{
	for (const FZoneShapeConnector& ShapeConnector : ShapeConnectors)
	{
		if (ShapeConnector.PointIndex == PointIndex)
		{
			return &ShapeConnector;
		}
	}
	return nullptr;
}

#if WITH_EDITOR
uint32 UZoneShapeComponent::GetShapeHash() const
{
	FZoneGraphObjectCRC32 Archive;
	const uint32 ShapeHash = Archive.Crc32(const_cast<UObject*>((const UObject*)this), 0);

	const FTransform Transform = GetComponentTransform();
	const uint32 TransformHash = GetTypeHash(Transform);

	return HashCombine(ShapeHash, TransformHash);
}
#endif

void UZoneShapeComponent::GetSplineLaneProfile(FZoneLaneProfile& OutLaneProfile) const
{
	if (ensure(ShapeType == FZoneShapeType::Spline))
	{
		if (const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>())
		{
			if (const FZoneLaneProfile* CurrentLaneProfile = ZoneGraphSettings->GetLaneProfileByRef(LaneProfile))
			{
				OutLaneProfile = *CurrentLaneProfile;
				return;
			}
		}
	}

	OutLaneProfile = FZoneLaneProfile();
}

void UZoneShapeComponent::GetPolygonLaneProfiles(TArray<FZoneLaneProfile>& OutLaneProfiles) const
{
	if (ensure(ShapeType == FZoneShapeType::Polygon))
	{
		if (const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>())
		{
			for (const FZoneShapePoint& Point : Points)
			{
				FZoneLaneProfileRef TemplateRef;

				if (Point.Type == FZoneShapePointType::LaneProfile)
				{
					if (Point.LaneProfile == FZoneShapePoint::InheritLaneProfile)
					{
						TemplateRef = LaneProfile;
					}
					else
					{
						if (ensure(Point.LaneProfile < PerPointLaneProfiles.Num()))
						{
							TemplateRef = PerPointLaneProfiles[Point.LaneProfile];
						}
					}
				}
				else
				{
					TemplateRef = LaneProfile;
				}

				if (const FZoneLaneProfile* CurrentLaneProfile = ZoneGraphSettings->GetLaneProfileByRef(TemplateRef))
				{
					OutLaneProfiles.Add(*CurrentLaneProfile);
				}
				else
				{
					OutLaneProfiles.AddDefaulted();
				}
			}
		}
	}

}

#if WITH_EDITOR
void UZoneShapeComponent::PostEditImport()
{
	Super::PostEditImport();

	UpdateShape();
	
	ShapeDataChangedEvent.Broadcast();

	if (UZoneGraphSubsystem* ZoneGraph = UWorld::GetSubsystem<UZoneGraphSubsystem>(GetWorld()))
	{
		ZoneGraph->GetBuilder().OnZoneShapeComponentChanged(*this);
	}
}

void UZoneShapeComponent::PostEditUndo()
{
	Super::PostEditUndo();

	ShapeDataChangedEvent.Broadcast();

	if (UZoneGraphSubsystem* ZoneGraph = UWorld::GetSubsystem<UZoneGraphSubsystem>(GetWorld()))
	{
		ZoneGraph->GetBuilder().OnZoneShapeComponentChanged(*this);
	}
}

void UZoneShapeComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.MemberProperty && PropertyChangedEvent.Property)
	{
		if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UZoneShapeComponent, ShapeType))
		{
			if (ShapeType == FZoneShapeType::Spline)
			{
				// Convert lane points to sharp points on spline.
				for (FZoneShapePoint& Point : Points)
				{
					if (Point.Type == FZoneShapePointType::LaneProfile)
					{
						Point.Type = FZoneShapePointType::Sharp;
						Point.LaneProfile = FZoneShapePoint::InheritLaneProfile;
					}
				}
				PerPointLaneProfiles.Reset();
			}
		}
	}

	UpdateShape();

	ShapeDataChangedEvent.Broadcast();

	if (UZoneGraphSubsystem* ZoneGraph = UWorld::GetSubsystem<UZoneGraphSubsystem>(GetWorld()))
	{
		ZoneGraph->GetBuilder().OnZoneShapeComponentChanged(*this);
	}
}

void UZoneShapeComponent::OnLaneProfileChanged(const FZoneLaneProfileRef& ChangedLaneProfileRef)
{
	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
	if (!ZoneGraphSettings)
	{
		return;
	}

	// First check if any template has been affected.
	bool bNeedsUpdate = false;

	if (LaneProfile == ChangedLaneProfileRef)
	{
		bNeedsUpdate = true;
	}

	if (!bNeedsUpdate)
	{
		for (const FZoneLaneProfileRef& PointLaneProfile : PerPointLaneProfiles)
		{
			if (PointLaneProfile == ChangedLaneProfileRef)
			{
				bNeedsUpdate = true;
				break;
			}
		}
	}

	if (bNeedsUpdate)
	{
		Modify();

		UpdateShape();

		if (UZoneGraphSubsystem* ZoneGraph = UWorld::GetSubsystem<UZoneGraphSubsystem>(GetWorld()))
		{
			ZoneGraph->GetBuilder().OnZoneShapeComponentChanged(*this);
		}
	}
}

#endif // WITH_EDITOR


#if !UE_BUILD_SHIPPING
FPrimitiveSceneProxy* UZoneShapeComponent::CreateSceneProxy()
{
	class FZoneShapeSceneProxy final : public FPrimitiveSceneProxy
	{
	public:
		virtual SIZE_T GetTypeHash() const override
		{
			static size_t UniquePointer;
			return reinterpret_cast<size_t>(&UniquePointer);
		}

		FZoneShapeSceneProxy(const UZoneShapeComponent& InComponent)
			: FPrimitiveSceneProxy(&InComponent)
		{
			ViewFlagIndex = uint32(FEngineShowFlags::FindIndexByName(TEXT("ZoneGraph")));

#if WITH_EDITOR
			if (UZoneGraphSubsystem* ZoneGraph = UWorld::GetSubsystem<UZoneGraphSubsystem>(InComponent.GetWorld()))
			{
				ZoneGraph->GetBuilder().BuildSingleShape(InComponent, FMatrix::Identity, ZoneStorage);
				ZoneStorage.DataHandle = FZoneGraphDataHandle(0xffff, 0xffff); // Give a valid handle so that the drawing happens correctly.
			}
#endif
			Connectors = InComponent.GetShapeConnectors();
			Connections = InComponent.GetConnectedShapes();
		}

		virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_ZoneShapeSceneProxy_GetDynamicMeshElements);

			if (IsSelected())
			{
				return;
			}

			static constexpr float DepthBias = 0.0001f;	// Little bias helps to make the lines visible when directly on top of geometry.
			static constexpr float LaneLineThickness = 2.0f;
			static constexpr float BoundaryLineThickness = 0.0f;

			float ShapeMaxDrawDistance = MAX_flt;
			if (const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>())
			{
				ShapeMaxDrawDistance = ZoneGraphSettings->GetShapeMaxDrawDistance();
			}

			const float CombinedMaxDrawDistance = FMath::Min(ShapeMaxDrawDistance, GetMaxDrawDistance());
			const float MinDrawDistanceSqr = FMath::Square(GetMinDrawDistance());
			const float MaxDrawDistanceSqr = FMath::Square(CombinedMaxDrawDistance);
			const float FadeDrawDistanceSqr = FMath::Square(CombinedMaxDrawDistance * 0.9f);
			const float DetailDrawDistanceSqr = FMath::Square(CombinedMaxDrawDistance * 0.5f);

			const FMatrix& LocalToWorld = GetLocalToWorld();
			const FVector ZoneCenter = GetBounds().Origin;

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if (VisibilityMap & (1 << ViewIndex))
				{
					const FSceneView* View = Views[ViewIndex];
					FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);

					// Taking into account the min and maximum drawing distance
					const float DistanceSqr = FVector::DistSquared(ZoneCenter, View->ViewMatrices.GetViewOrigin());
					if (DistanceSqr < MinDrawDistanceSqr || DistanceSqr > MaxDrawDistanceSqr)
					{
						continue;
					}

					// Only draw details close to camera
					const bool bDrawDetails = DistanceSqr < DetailDrawDistanceSqr;
					// Fade visualization before culling.
					const float Alpha = 1.0f - FMath::Clamp((DistanceSqr - FadeDrawDistanceSqr) / (MaxDrawDistanceSqr - FadeDrawDistanceSqr), 0.0f, 1.0f);
					// We have only one zone in the storage, created in constructor.
					constexpr int32 ZoneIndex = 0;

					// Draw boundary
					UE::ZoneGraph::RenderingUtilities::DrawZoneBoundary(ZoneStorage, ZoneIndex, PDI, LocalToWorld, BoundaryLineThickness, DepthBias, Alpha);

					// Draw Lanes
					UE::ZoneGraph::RenderingUtilities::DrawZoneLanes(ZoneStorage, ZoneIndex, PDI, LocalToWorld, LaneLineThickness, DepthBias, Alpha, bDrawDetails);

					if (bDrawDetails)
					{
						// Draw connectors
						UE::ZoneGraph::RenderingUtilities::DrawZoneShapeConnectors(Connectors, Connections, PDI, LocalToWorld, DepthBias);
					}

				}
			}
		}

		virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
		{
			FPrimitiveViewRelevance Result;
			Result.bDrawRelevance = !IsSelected() && IsShown(View) && View->Family->EngineShowFlags.GetSingleFlag(ViewFlagIndex);
			Result.bDynamicRelevance = true;
			Result.bShadowRelevance = IsShadowCast(View);
			Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
			Result.bSeparateTranslucency = Result.bNormalTranslucency = true;
			return Result;
		}

		virtual uint32 GetMemoryFootprint(void) const override { return sizeof * this + GetAllocatedSize(); }
		uint32 GetAllocatedSize(void) const { return (uint32)FPrimitiveSceneProxy::GetAllocatedSize(); }

	private:
		TArray<FZoneShapeConnector> Connectors;
		TArray<FZoneShapeConnection> Connections;
		FZoneGraphStorage ZoneStorage;
		uint32 ViewFlagIndex = 0;
	};

	return new FZoneShapeSceneProxy(*this);
}

#endif // !UE_BUILD_SHIPPING

