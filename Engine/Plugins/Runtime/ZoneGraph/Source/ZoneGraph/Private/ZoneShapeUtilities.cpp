// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZoneShapeUtilities.h"
#include "Curves/BezierUtilities.h"
#include "Algo/Reverse.h"
#include "HAL/IConsoleManager.h"
#include "ZoneGraphSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ZoneShapeUtilities)

namespace UE::ZoneGraph::Debug {

	bool bRemoveOverlap = true;
	bool bRemoveSameDestination = true;
	bool bFillEmptyDestination = true;

	FAutoConsoleVariableRef VarsGeneration[] = {
		FAutoConsoleVariableRef(TEXT("ai.debug.zonegraph.generation.RemoveOverlap"), bRemoveOverlap, TEXT("Remove Overlapping lanes.")),
		FAutoConsoleVariableRef(TEXT("ai.debug.zonegraph.generation.RemoveSameDestination"), bRemoveSameDestination, TEXT("Remove merging lanes leading to same destination.")),
		FAutoConsoleVariableRef(TEXT("ai.debug.zonegraph.generation.FillEmptyDestination"), bFillEmptyDestination, TEXT("Fill stray empty destination lanes.")),
	};

} // UE::ZoneGraph::Debug

namespace UE::ZoneShape::Utilities {

// Normalizes an angle by making it in range -PI..PI. Angle in radians.
static float WrapAngle(const float Angle)
{
	float WrappedAngle = FMath::Fmod(Angle, PI*2.0f);

	if (WrappedAngle > PI)
	{
		WrappedAngle -= PI*2.0f;
	}

	if (WrappedAngle < -PI)
	{
		WrappedAngle += PI*2.0f;
	}

	return WrappedAngle;
}

// Calculates turning angle from start to end, CCW controls if the turn will be counter-clock-wise or clock-wise.
// If the delta angle is less than DeltaAngleDeadZone it is clamped to zero.
// Angles in radians.
static float TurnAngle(const float AngleStart, const float AngleEnd, const bool bCCW, const float DeltaAngleDeadZone = FMath::DegreesToRadians(0.1f))
{
	float DeltaAngle = WrapAngle(AngleEnd - AngleStart);

	if (FMath::Abs(DeltaAngle) < DeltaAngleDeadZone)
	{
		DeltaAngle = 0.0f;
	}

	if (bCCW && DeltaAngle > 0.0f)
	{
		DeltaAngle -= PI*2;
	}

	if (!bCCW && DeltaAngle < 0.0f)
	{
		DeltaAngle += PI*2;
	}
	
	return DeltaAngle;
}

// Rotates direction vector 90 counter clockwise.
static FVector2D Rotate90CCW(const FVector2D Vec)
{
	return FVector2D(Vec.Y, -Vec.X);
}

// Intersects two rays. If hit, returns true and distance along each ray.
// Assumes that the directions are normalized.
static bool IntersectRayRay2D(const FVector2D PosA, const FVector2D DirA, const FVector2D PosB, const FVector2D DirB, float& OutDistanceA, float& OutDistanceB)
{
	const FVector2D RelPos = PosA - PosB;
	const float Denom = FVector2D::CrossProduct(DirA, DirB);
	const float NumerA = FVector2D::CrossProduct(DirB, RelPos);
	const float NumerB = FVector2D::CrossProduct(DirA, RelPos);

	// Almost coincident case, any value will do.
	if (FMath::IsNearlyZero(NumerA, KINDA_SMALL_NUMBER) && FMath::IsNearlyZero(NumerB, KINDA_SMALL_NUMBER) && FMath::IsNearlyZero(Denom, KINDA_SMALL_NUMBER))
	{
		OutDistanceA = 0.0f;
		OutDistanceB = 0.0f;
		return true;
	}

	// Almost parallel case, no intersection.
	if (FMath::IsNearlyZero(Denom))
	{
		OutDistanceA = 0.0f;
		OutDistanceB = 0.0f;
		return false;
	}

	OutDistanceA = NumerA / Denom;
	OutDistanceB = NumerB / Denom;

	return true;
}

// Describes a circular arc. Angles in radians.
struct FArc
{
	// Return arc length
	float ArcLength() const
	{
		const float ArcFraction = FMath::Abs(DeltaAngle / (PI * 2.f));
		return 2.0f * PI * Radius * ArcFraction;
	}

	// Evaluates a position on the arc, at normalized time T (0..1)
	FVector2D Eval(const float T) const
	{
		const float Angle = StartAngle + DeltaAngle * T;
		const FVector2D Dir(FMath::Cos(Angle), FMath::Sin(Angle));
		return Center + Dir * Radius;
	}

	// Evaluates a direction on the arc, at normalized time T (0..1)
	FVector2D EvalDir(const float T) const
	{
		const float Angle = StartAngle + DeltaAngle * T;
		const float DirAngle = DeltaAngle < 0.0f ? (Angle - PI/2.0f) : (Angle + PI/2.0f);
		return FVector2D(FMath::Cos(DirAngle), FMath::Sin(DirAngle));
	}

    FVector2D Center = FVector2D::ZeroVector;
    float Radius = 0.0f;
    float StartAngle = 0.0f;	// Radians.
    float DeltaAngle = 0.0f;	// Radians.
};

// Describes a path using segment-arc-segment-arc-segment.
// Can be used to store a Dubin's path, or simple path with a rounded corner. 
struct FDubinsPath
{
	// Returns length of the path.
	float PathLength() const
	{
		const FVector2D StartArcStart = StartArc.Eval(0.0f);
		const FVector2D StartArcEnd = StartArc.Eval(1.0f);
		const FVector2D EndArcStart = EndArc.Eval(0.0f);
		const FVector2D EndArcEnd = EndArc.Eval(1.0f);

		return	FVector2D::Distance(StartPos, StartArcStart) +
				StartArc.ArcLength() +
				FVector2D::Distance(StartArcEnd, EndArcStart) +
				EndArc.ArcLength() + 
				FVector2D::Distance(EndArcEnd, EndPos);
	}
	
    FVector2D StartPos = FVector2D::ZeroVector;
    FVector2D StartDir = FVector2D::ZeroVector;
    FVector2D EndPos = FVector2D::ZeroVector;
    FVector2D EndDir = FVector2D::ZeroVector;
    FArc StartArc;
    FArc EndArc; 
};

// Returns number of divisions for an arc, so that the max distance between cord segment and arc is 'Tolerance'.
static int32 CalculateArcDivs(float const Radius, const float DeltaAngle, const float Tolerance)
{
	static const int32 MinPoints = 2;
	if (Radius < KINDA_SMALL_NUMBER)
	{
		return MinPoints;
	}
	const float Dist = FMath::Acos(1.0f - Tolerance / Radius);
	const float ArcFraction = FMath::Abs(DeltaAngle / (PI * 2.f));
	return FMath::Max(MinPoints, FMath::CeilToInt((PI * 2.f / Dist) * ArcFraction));
}

// Calculates Dubins Circle-Segment-Circle path. Will shrink the radii to make the solution valid.
static void CalculatePathCircleSegCircle(const FVector2D StartPos, const FVector2D StartDir, float StartRadius, const bool bStartTurnCCW,
										 const FVector2D EndPos, const FVector2D EndDir, float EndRadius, const bool bEndTurnCCW,
										 FDubinsPath& OutPath)
{
	// When the path is almost straight (directions about the same, and aligned to the line from Start to End),
	// the turn from StartAngle and TangentAngle can be in opposite direction than what the bStartTurnCCW/bEndTurnCCW
	// expect, and that will result a full loop.
	// The TurnDeadZoneAngle controls how small angles are clamped to zero to avoid that.
	static const float TurnDeadZoneAngle = FMath::DegreesToRadians(2.0f);

	const FVector2D StartNorm = bStartTurnCCW ? Rotate90CCW(StartDir) : -Rotate90CCW(StartDir);
	const FVector2D EndNorm = bEndTurnCCW ? Rotate90CCW(EndDir) : -Rotate90CCW(EndDir);
	const float StartAngle = FMath::Atan2(-StartNorm.Y, -StartNorm.X);
	const float EndAngle = FMath::Atan2(-EndNorm.Y, -EndNorm.X);
	const float MinRadius = FMath::Min(StartRadius, EndRadius);
	const float MaxRadius = FMath::Max(StartRadius, EndRadius);

	// We'll shrink the radii to always find a valid result.
	// TODO: There are analytical solutions to these, could not get them to work for the time being. Binary search will suffice for the time being.
	bool bIsConstrained = false;
	if (bStartTurnCCW == bEndTurnCCW)
	{
		// Parallel tangents, smaller of the circles cannot be embedded in the bigger one.
		const FVector2D InitialStartCenter = StartPos + StartNorm * StartRadius;
		const FVector2D InitialEndCenter = EndPos + EndNorm * EndRadius;
		const float InitialCentersDist = FVector2D::Distance(InitialStartCenter, InitialEndCenter);

		if (InitialCentersDist <= (MaxRadius - MinRadius))
		{
			float Scale = 0.5f;
			float Fraction = 0.25f;
			for (int32 Iter = 0; Iter < 8; Iter++)
			{
				const FVector2D StartCenter = StartPos + StartNorm * StartRadius * Scale;
				const FVector2D EndCenter = EndPos + EndNorm * EndRadius * Scale;
				const float MinRadiusScaled = MinRadius * Scale;
				const float MaxRadiusScaled = MaxRadius * Scale;
				const float CentersDist = FVector2D::Distance(StartCenter, EndCenter);
				if (CentersDist <= (MaxRadiusScaled - MinRadiusScaled))
				{
					Scale -= Fraction;
				}
				else
				{
					Scale += Fraction;
				}
				Fraction *= 0.5f;
			}
			StartRadius *= Scale;
			EndRadius *= Scale;
			bIsConstrained = true;
		}
	}
	else
	{
		// Crossing tangents, circles cannot overlap.
		const FVector2D InitialStartCenter = StartPos + StartNorm * StartRadius;
		const FVector2D InitialEndCenter = EndPos + EndNorm * EndRadius;
		const float InitialCentersDist = FVector2D::Distance(InitialStartCenter, InitialEndCenter);

		if (InitialCentersDist <= (MaxRadius + MinRadius))
		{
			float Scale = 0.5f;
			float Fraction = 0.25f;
			for (int32 Iter = 0; Iter < 8; Iter++)
			{
				const FVector2D StartCenter = StartPos + StartNorm * StartRadius * Scale;
				const FVector2D EndCenter = EndPos + EndNorm * EndRadius * Scale;
				const float MinRadiusScaled = MinRadius * Scale;
				const float MaxRadiusScaled = MaxRadius * Scale;
				const float CentersDist = FVector2D::Distance(StartCenter, EndCenter);
				if (CentersDist <= (MaxRadiusScaled + MinRadiusScaled))
				{
					Scale -= Fraction;
				}
				else
				{
					Scale += Fraction;
				}
				Fraction *= 0.5f;
			}
			StartRadius *= Scale;
			EndRadius *= Scale;
			bIsConstrained = true;
		}
	}
	
	const FVector2D StartCenter = StartPos + StartNorm * StartRadius;
	const FVector2D EndCenter = EndPos + EndNorm * EndRadius;
	const FVector2D CentersDir = (EndCenter - StartCenter).GetSafeNormal();
	const float CentersDist = FVector2D::Distance(StartCenter, EndCenter);
	const float CentersAngle = FMath::Atan2(CentersDir.Y, CentersDir.X);

	OutPath.StartPos = StartPos;
	OutPath.StartDir = StartDir;
	OutPath.StartArc.Center = StartCenter;
	OutPath.StartArc.Radius = StartRadius;
	OutPath.EndArc.Center = EndCenter;
	OutPath.EndArc.Radius = EndRadius;
	OutPath.EndPos = EndPos;
	OutPath.EndDir = EndDir;

	if (bStartTurnCCW == bEndTurnCCW)
	{
		// Outer parallel tangents 
		float TangentAngle = 0.0f;
		const float DeltaRadius = StartRadius - EndRadius;
		if (FMath::IsNearlyEqual(CentersDist, FMath::Abs(DeltaRadius), 0.1f) || bIsConstrained)
		{
			// Circles are nearly touching, we'll get zero length tangent.
			TangentAngle = DeltaRadius < 0 ? CentersAngle : WrapAngle(CentersAngle + PI * 2.0f);
		}
		else
		{
			const float Base = FMath::Sqrt(FMath::Max(0.0f, FMath::Square(CentersDist) - FMath::Square(DeltaRadius)));
			const float TangentDeltaAngle = FMath::Atan2(Base, DeltaRadius);
			TangentAngle = bStartTurnCCW ? (CentersAngle + TangentDeltaAngle) : (CentersAngle - TangentDeltaAngle);
		}
		OutPath.StartArc.StartAngle = StartAngle;
		OutPath.StartArc.DeltaAngle = TurnAngle(StartAngle, TangentAngle, bStartTurnCCW, TurnDeadZoneAngle);
		OutPath.EndArc.StartAngle = TangentAngle;
		OutPath.EndArc.DeltaAngle = TurnAngle(TangentAngle, EndAngle, bEndTurnCCW, TurnDeadZoneAngle);
	}
	else
	{
		// Inner crossing tangents
		float TangentAngle = 0.0f;
		const float DeltaRadius = StartRadius + EndRadius;
		if (FMath::IsNearlyEqual(CentersDist, DeltaRadius, 0.1f) || bIsConstrained)
		{
			// Circles are nearly touching, we'll get zero length tangent.
			TangentAngle = CentersAngle;
		}
		else
		{
			const float Base = FMath::Sqrt(FMath::Max(0.0f, FMath::Square(CentersDist) - FMath::Square(DeltaRadius)));
			const float TangentDeltaAngle = FMath::Atan2(Base, DeltaRadius);
			TangentAngle = bStartTurnCCW ? (CentersAngle + TangentDeltaAngle) : (CentersAngle - TangentDeltaAngle);
		}
		
		OutPath.StartArc.StartAngle = StartAngle;
		OutPath.StartArc.DeltaAngle = TurnAngle(StartAngle, TangentAngle, bStartTurnCCW, TurnDeadZoneAngle);
		OutPath.EndArc.StartAngle = WrapAngle(TangentAngle + PI);
		OutPath.EndArc.DeltaAngle = TurnAngle(OutPath.EndArc.StartAngle, EndAngle, bEndTurnCCW, TurnDeadZoneAngle);
	}
}

// Calculates shortest Dubin's path between two points. Will shrink the radii to make the solution valid.
static void CalculateDubinsPath(const FVector2D StartPos, const FVector2D StartDir, const float StartRadius,
								const FVector2D EndPos, const FVector2D EndDir, const float EndRadius,
								FDubinsPath& OutPath)
{
	FDubinsPath Path;
	float ShortestPathLength = MAX_flt;

	// Test all CW/CCW turn combinations, pick shortest one.
	for (int32 StartIdx = 0; StartIdx < 2; StartIdx++)
	{
		for (int32 EndIdx = 0; EndIdx < 2; EndIdx++)
		{
			const bool bStartTurnCCW = StartIdx == 0;
			const bool bEndTurnCCW = EndIdx == 0;
			FDubinsPath TempPath;
			CalculatePathCircleSegCircle(StartPos, StartDir, StartRadius, bStartTurnCCW, EndPos, EndDir, EndRadius, bEndTurnCCW, TempPath);
			
			const float TempPathLength = TempPath.PathLength();
			if (TempPathLength < ShortestPathLength)
			{
				ShortestPathLength = TempPathLength;
				OutPath = TempPath;
			}
		}
	}
}

static bool CalculateIntersectingPath(const FVector2D StartPos, const FVector2D StartDir, const float StartRadius,
									  const FVector2D EndPos, const FVector2D EndDir, const float EndRadius,
									  const float MinTurnRadius, FDubinsPath& OutPath)
{
	// Require some deviation between the directions to reduce some near parallel/coincident corner cases.
	static const float MinAngleCos = FMath::Cos(FMath::DegreesToRadians(5.0f));
	if (FVector2D::DotProduct(StartDir, EndDir) > MinAngleCos)
	{
		return false;
	}

	// Find intersection between the start and end points, along their directions. The directions are point along the movement direction, so end is negated.
	float StartDistance = MAX_flt, EndDistance = MAX_flt;
	if (!IntersectRayRay2D(StartPos, StartDir, EndPos, -EndDir, StartDistance, EndDistance) && StartDistance > 0.0f && EndDistance > 0.0f)
	{
		return false;
	}

	// Find parameters to fit a circle at the intersection corner.
	const FVector2D Corner = StartPos + StartDir * StartDistance;
	const FVector2D StartCornerDir = -StartDir; // From corner to start point
	const FVector2D EndCornerDir = EndDir; // From corner to end point
	const FVector2D Bisector = (StartCornerDir + EndCornerDir).GetSafeNormal();
	const float CosAngle = FVector2D::DotProduct(StartCornerDir, EndCornerDir);
	const float HalfAngle = FMath::Acos(FMath::Clamp(CosAngle, -1.0f, 1.0f)) / 2.0f;
	const float EdgeSlope = 1.0f / FMath::Tan(HalfAngle);

	// The corner radius is blend between start and end radius, depending how close the intersection is to start or end point.
	const float StartEdgeOffset = StartRadius * EdgeSlope;
	const float EndEdgeOffset = EndRadius * EdgeSlope;
	const float StartEdgeDist = StartDistance - StartEdgeOffset; // At this distance we want the radius to be 'StartRadius'.
	const float EndEdgeDist = EndDistance - EndEdgeOffset; // At this distance we want the radius to be 'EndRadius'.
	float Radius = 0.0f;
	if (StartEdgeDist > 0.0f || EndEdgeDist > 0.0f)
	{
		// Use weighted sum when the circle tangents fits the available segments.
		const float Weight = FMath::Max(0.0f, StartEdgeDist) / (FMath::Max(0.0f, StartEdgeDist) + FMath::Max(0.0f, EndEdgeDist));
		Radius = FMath::Lerp(StartRadius, EndRadius, Weight);
	}
	else
	{
		Radius = StartDistance < EndDistance ? StartRadius : EndRadius;
	}
	
	// Calculate the offset along the segments starting from the corner where the edges touch the circle.
	float EdgeOffset = Radius * EdgeSlope;

	// TODO: this shrinking can lead to discontinuities when editing the curve manually.
	// If the tangent point would be outside the segments, try to clamp it to fit it.
	if (StartDistance < EdgeOffset)
	{
		Radius = StartDistance / EdgeSlope;
		EdgeOffset = StartDistance;
	}

	if (EndDistance < EdgeOffset)
	{
		Radius = EndDistance / EdgeSlope;
		EdgeOffset = EndDistance;
	}
	
	// Cannot violate the absolute minimum turning radius.
	if (Radius < MinTurnRadius)
	{
		return false;
	}
		
	StartDistance -= EdgeOffset;
	EndDistance -= EdgeOffset;

	// If the rounding would shoot out further than that, do not round.
	const float MaxRoundingDistance = FVector2D::Distance(StartPos, EndPos);
	if (StartDistance > MaxRoundingDistance || EndDistance > MaxRoundingDistance)
	{
		return false;
	}

	// Calculate final location of the turn.
	const FVector2D StartNorm = Rotate90CCW(StartDir);
	const bool bTurnCCW = FVector2D::DotProduct(StartNorm, EndPos - StartPos) > 0.0f;
	
	const float BisectorOffset = Radius / FMath::Sin(HalfAngle);
	const FVector2D Center = Corner + Bisector * BisectorOffset;
	const FVector2D StartEdgePos = StartPos + StartDir * StartDistance; 
	const FVector2D EndEdgePos = EndPos + -EndDir * EndDistance; 
	const FVector2D DirToStartEdgePos = StartEdgePos - Center;
	const FVector2D DirToEndEdgePos = EndEdgePos - Center;
	const float StartAngle = FMath::Atan2(DirToStartEdgePos.Y, DirToStartEdgePos.X);
	const float EndAngle = FMath::Atan2(DirToEndEdgePos.Y, DirToEndEdgePos.X);
	const float DeltaAngle = TurnAngle(StartAngle, EndAngle, bTurnCCW);
	
	OutPath.StartArc.Center = Center;
	OutPath.StartArc.Radius = Radius;
	OutPath.StartArc.StartAngle = StartAngle;
	OutPath.StartArc.DeltaAngle = DeltaAngle * 0.5f;

	OutPath.EndArc.Center = Center;
	OutPath.EndArc.Radius = Radius;
	OutPath.EndArc.StartAngle = StartAngle + DeltaAngle * 0.5f;
	OutPath.EndArc.DeltaAngle = DeltaAngle * 0.5f;

	OutPath.StartPos = StartPos;
	OutPath.StartDir = StartDir;
	OutPath.EndPos = EndPos;
	OutPath.EndDir = EndDir;

	return true;
}

// Calculates path from Start to End using line segments and arcs. If possible, calculate a path with one turn based on the intersection of the
// directions vectors. As fallback, calculate Dubin's path, that is, arc at each end connected by segment. If thre is no solution
// with the given radii, they will be shrank to find a valid solution. The paths should should not circle back, but stay between the control
// points similar to Bezier curves. Start and
// Start and end directions are movement directions.
static void CalculateIntersectingOrDubinsPath(const FVector2D StartPos, const FVector2D StartDir, const float StartRadiusCCW, const float StartRadiusCW,
											  const FVector2D EndPos, const FVector2D EndDir, const float EndRadiusCCW, const float EndRadiusCW,
											  const float MinTurnRadius, FDubinsPath& OutPath)
{
	const FVector2D StartNorm = Rotate90CCW(StartDir);
	const FVector2D EndNorm = Rotate90CCW(EndDir);

	const bool bStartCCW = FVector2D::DotProduct(StartNorm, EndPos - StartPos) < 0.0f;
	const bool bEndCCW = FVector2D::DotProduct(EndNorm, StartPos - EndPos) < 0.0f;

	const float StartRadius = bStartCCW ? StartRadiusCCW : StartRadiusCW;
	const float EndRadius = bEndCCW ? EndRadiusCCW : EndRadiusCW;

	if (!CalculateIntersectingPath(StartPos, StartDir, StartRadius, EndPos, EndDir, EndRadius, MinTurnRadius, OutPath))
	{
		CalculateDubinsPath(StartPos, StartDir, StartRadius, EndPos, EndDir, EndRadius, OutPath);
	}
}

static void SimplifyShape(TArray<FVector>& InOutPoints, const int32 StartIdx, const float Tolerance)
{
	const float ToleranceSqr = FMath::Square(Tolerance);
	int32 Index = StartIdx;
	while ((Index + 2) < InOutPoints.Num())
	{
		// Remove Mid if it is closer than Tolerance from the segment Start-End
		const int32 Start = Index;
		const int32 Mid = Index + 1;
		const int32 End = Index + 2;
		if (FMath::PointDistToSegmentSquared(InOutPoints[Mid], InOutPoints[Start], InOutPoints[End]) < ToleranceSqr)
		{
			// Remove Mid
			InOutPoints.RemoveAt(Mid);
		}
		else
		{
			Index++;
		}
	}
}

static void TessellateDubinsPath(const FDubinsPath& Path, TArray<FVector>& OutPoints, const float TessTolerance)
{
	const int32 PointsBegin = OutPoints.Num(); 
	
	OutPoints.Add(FVector(Path.StartPos, 0.0f));

	const int32 StartDivs = CalculateArcDivs(Path.StartArc.Radius, Path.StartArc.DeltaAngle, TessTolerance);
	for (int32 Index = 0; Index < StartDivs; Index++)
	{
		const float Alpha = (float)(Index) / (float)(StartDivs-1);
		OutPoints.Add(FVector(Path.StartArc.Eval(Alpha), 0.0f));
	}

	const int32 EndDivs = CalculateArcDivs(Path.EndArc.Radius, Path.EndArc.DeltaAngle, TessTolerance);
	for (int32 Index = 0; Index < EndDivs; Index++)
	{
		const float Alpha = (float)(Index) / (float)(EndDivs-1);
		OutPoints.Add(FVector(Path.EndArc.Eval(Alpha), 0.0f));
	}

	OutPoints.Add(FVector(Path.EndPos, 0.0f));

	// Remove degenerate points.
	// @todo: Small segments can have a large affect on how the path behaves when smoothed. The 0.1 is fudge factor here to prevent small
	// segments from being eaten.A proper solution would be to always bevel the corner so that it does not get simplified by TessTolerance.
	SimplifyShape(OutPoints, PointsBegin, FMath::Max(KINDA_SMALL_NUMBER, TessTolerance * 0.1f));
}


struct FShapePoint
{
	FShapePoint() = default;
	FShapePoint(const FVector& InPosition, const FVector& InUp) : Position(InPosition), Up(InUp) {}

	FVector Position = FVector::ZeroVector;
	FVector Up = FVector::UpVector;
	FVector Right = FVector::RightVector;
};

static void RemoveDegenerateSegments(TArray<FShapePoint>& Points, const bool bClosed)
{
	for (int32 i = 0; i < Points.Num() - 1; i++)
	{
		if (Points.Num() <= 2)
		{
			break;
		}
			
		if (FVector::DistSquared(Points[i].Position, Points[i + 1].Position) < SMALL_NUMBER)
		{
			Points.RemoveAt(i + 1);
			i--;
		}
	}

	if (bClosed)
	{
		if (Points.Num() > 2 && FVector::DistSquared(Points[0].Position, Points.Last().Position) < SMALL_NUMBER)
		{
			Points.Pop();
		}
	}
}

static void SimplifyShape(TArray<FShapePoint>& Points, const float Tolerance)
{
	const float ToleranceSqr = FMath::Square(Tolerance);
	int32 Index = 0;
	while ((Index + 2) < Points.Num())
	{
		// Remove Mid if it is closer than Tolerance from the segment Start-End
		const int32 Start = Index;
		const int32 Mid = Index + 1;
		const int32 End = Index + 2;
		if (FMath::PointDistToSegmentSquared(Points[Mid].Position, Points[Start].Position, Points[End].Position) < ToleranceSqr)
		{
			// Remove Mid
			Points.RemoveAt(Mid);
		}
		else
		{
			Index++;
		}
	}
}

void GetCubicBezierPointsFromShapeSegment(const FZoneShapePoint& StartShapePoint, const FZoneShapePoint& EndShapePoint, const FMatrix& LocalToWorld,
										  FVector& OutStartPoint, FVector& OutStartControlPoint, FVector& OutEndControlPoint, FVector& OutEndPoint)
{
	if (StartShapePoint.Type == FZoneShapePointType::LaneProfile)
	{
		// The profile points inwards, so out is left.
		OutStartPoint = LocalToWorld.TransformPosition(StartShapePoint.GetLaneProfileLeft());
		OutStartControlPoint = OutStartPoint;
	}
	else
	{
		OutStartPoint = LocalToWorld.TransformPosition(StartShapePoint.Position);
		OutStartControlPoint = LocalToWorld.TransformPosition(StartShapePoint.GetOutControlPoint());
	}

	if (EndShapePoint.Type == FZoneShapePointType::LaneProfile)
	{
		OutEndPoint = LocalToWorld.TransformPosition(EndShapePoint.GetLaneProfileRight());
		OutEndControlPoint = OutEndPoint;
	}
	else
	{
		// The profile points inwards, so in is right.
		OutEndControlPoint = LocalToWorld.TransformPosition(EndShapePoint.GetInControlPoint());
		OutEndPoint = LocalToWorld.TransformPosition(EndShapePoint.Position);
	}
}

static void FlattenSplineSegments(TConstArrayView<FZoneShapePoint> Points, bool bClosed, const FMatrix& LocalToWorld, const float Tolerance, TArray<FShapePoint>& OutPoints)
{
	// Tessellate points.
	const int32 NumPoints = Points.Num();
	int StartIdx = bClosed ? (NumPoints - 1) : 0;
	int Idx = bClosed ? 0 : 1;


	if (!bClosed)
	{
		const FZoneShapePoint& StartShapePoint = Points[StartIdx];
		if (StartShapePoint.Type != FZoneShapePointType::LaneProfile)
		{
			const FVector WorldPosition = LocalToWorld.TransformPosition(StartShapePoint.Position);
			const FVector WorldUp = LocalToWorld.TransformVector(StartShapePoint.Rotation.RotateVector(FVector::UpVector)).GetSafeNormal();
			OutPoints.Add(FShapePoint(WorldPosition, WorldUp));
		}
	}

	TArray<FVector> TempPoints;
	TArray<float> TempProgression;

	while (Idx < NumPoints)
	{
		FVector StartPosition(ForceInitToZero), StartControlPoint(ForceInitToZero), EndControlPoint(ForceInitToZero), EndPosition(ForceInitToZero);
		GetCubicBezierPointsFromShapeSegment(Points[StartIdx], Points[Idx], LocalToWorld, StartPosition, StartControlPoint, EndControlPoint, EndPosition);

		// TODO: The Bezier tessellation does not take into account the roll when calculating tolerance.
		// Maybe we should have a templated version which would do the up axis interpolation too.

		TempPoints.Reset();
		if (Points[StartIdx].Type == FZoneShapePointType::LaneProfile)
		{
			TempPoints.Add(StartPosition);
		}
		UE::CubicBezier::Tessellate(TempPoints, StartPosition, StartControlPoint, EndControlPoint, EndPosition, Tolerance);

		TempProgression.SetNum(TempPoints.Num());

		// Interpolate up vector for points
		float TotalDist = FVector::Dist(StartPosition, TempPoints[0]);
		for (int32 i = 0; i < TempPoints.Num() - 1; i++)
		{
			TempProgression[i] = TotalDist;
			TotalDist += FVector::Dist(TempPoints[i], TempPoints[i + 1]);
		}
		TempProgression[TempProgression.Num() - 1] = TotalDist;

		// Add points and interpolate up axis
		const FQuat StartRotation = Points[StartIdx].Rotation.Quaternion();
		const FQuat EndRotation = Points[Idx].Rotation.Quaternion();
		for (int32 i = 0; i < TempPoints.Num(); i++)
		{
			const float Alpha = TempProgression[i] / TotalDist;
			FQuat Rotation = FMath::Lerp(StartRotation, EndRotation, Alpha);
			const FVector WorldUp = LocalToWorld.TransformVector(Rotation.RotateVector(FVector::UpVector)).GetSafeNormal();
			OutPoints.Add(FShapePoint(TempPoints[i], WorldUp));
		}

		StartPosition = EndPosition;
		StartIdx = Idx;
		Idx++;
	}
}

static void CalculateLaneProgression(TConstArrayView<FVector> LanePoints, const int32 PointsBegin, const int32 PointsEnd, TArray<float>& OutLaneProgression)
{
	float TotalDist = 0.0f;
	for (int32 i = PointsBegin; i < PointsEnd - 1; i++)
	{
		OutLaneProgression[i] = TotalDist;
		TotalDist += FVector::Dist(LanePoints[i], LanePoints[i + 1]);
	}
	OutLaneProgression[PointsEnd - 1] = TotalDist;
}

static void CalculateStartAndEndNormals(TConstArrayView<FZoneShapePoint> Points, const FMatrix& LocalToWorld, FVector& OutStartNormal, FVector& OutEndNormal)
{
	const int NumPoints = Points.Num();
	if (NumPoints < 2)
	{
		OutStartNormal = FVector::RightVector;
		OutEndNormal = FVector::RightVector;
		return;
	}

	OutStartNormal = LocalToWorld.TransformVector(Points[0].Rotation.RotateVector(FVector::RightVector).GetSafeNormal());
	OutEndNormal = LocalToWorld.TransformVector(Points.Last().Rotation.RotateVector(FVector::RightVector).GetSafeNormal());
}

static void CalculateEdgeNormals(TArrayView<FShapePoint> Points, TArray<FVector>& OutNormals)
{
	check(Points.Num() > 1);

	const int32 NumPoints = Points.Num();

	OutNormals.SetNum(NumPoints);

	for (int32 i = 0; i < NumPoints - 1; i++)
	{
		FShapePoint& Point = Points[i];
		FShapePoint& NextPoint = Points[i + 1];
		const FVector Forward = NextPoint.Position - Point.Position;
		OutNormals[i] = FVector::CrossProduct(Point.Up, Forward).GetSafeNormal();
	}
	OutNormals[NumPoints - 1] = OutNormals[NumPoints - 2];
}

static void CalculateMiters(TArrayView<FShapePoint> Points, TConstArrayView<FVector> EdgeNormals)
{
	check(Points.Num() == EdgeNormals.Num());

	const int32 NumPoints = Points.Num();

	Points[0].Right = EdgeNormals[0];

	for (int32 i = 1; i < NumPoints - 1; i++)
	{
		// TODO: should we do bevel or round miters?
		// For in between points we calculate mitered normal, that is to keep the segment with between vertices constant.
		FVector AvgNormal = (EdgeNormals[i - 1] + EdgeNormals[i]) * 0.5f;
		const float LenSqr = AvgNormal.SizeSquared();
		if (LenSqr > KINDA_SMALL_NUMBER)
		{
			static const float MaxMiter = 20.0f;	// Avoid crazy overshoot on very sharp corners.
			float Scale = FMath::Min(1.0f / LenSqr, MaxMiter);
			if (Scale < (1.0f - KINDA_SMALL_NUMBER))
			{
				Scale = 1.0f;
			}
			Points[i].Right = AvgNormal * Scale;
		}
		else
		{
			// Combined normal is degenerate (i.e.edge normals are opposite), use current segments normal as is.
			Points[i].Right = EdgeNormals[i];
		}
	}

	Points[NumPoints - 1].Right = EdgeNormals[NumPoints - 1];
}

float GetMinLaneProfileTessellationTolerance(const FZoneLaneProfile& LaneProfile, const FZoneGraphTagMask ZoneTags, const FZoneGraphBuildSettings& BuildSettings)
{
	float TessTolerance = MAX_flt;
	for (const FZoneLaneDesc& Lane : LaneProfile.Lanes)
	{
		const float LaneTessTolerance = BuildSettings.GetLaneTessellationTolerance(Lane.Tags | ZoneTags);
		TessTolerance = FMath::Min(TessTolerance, LaneTessTolerance);
	}
	return TessTolerance;
}

static void AddAdjacentLaneLinks(const int32 CurrentLaneIndex, const int32 LaneDescIndex, const TArray<FZoneLaneDesc>& LaneDescs, TArray<FZoneShapeLaneInternalLink>& OutInternalLinks)
{
	const int32 NumLanes = LaneDescs.Num();
	const FZoneLaneDesc& LaneDesc = LaneDescs[LaneDescIndex];

	// Assign left/right based on current lane direction. Lanes are later arranged so that they all point forward.
	EZoneLaneLinkFlags PrevLinkFlags = EZoneLaneLinkFlags::None;
	EZoneLaneLinkFlags NextLinkFlags = EZoneLaneLinkFlags::None;
	if (LaneDesc.Direction == EZoneLaneDirection::Forward)
	{
		NextLinkFlags = EZoneLaneLinkFlags::Left;
		PrevLinkFlags = EZoneLaneLinkFlags::Right;
	}
	else if (LaneDesc.Direction == EZoneLaneDirection::Backward)
	{
		NextLinkFlags = EZoneLaneLinkFlags::Right;
		PrevLinkFlags = EZoneLaneLinkFlags::Left;
	}
	else if (LaneDesc.Direction == EZoneLaneDirection::None)
	{
		ensureMsgf(false, TEXT("This function should not be called on spacer/none lanes."));
		return;
	}
	else
	{
		ensureMsgf(false, TEXT("Lane direction %d not implemented."), int32(LaneDesc.Direction));
	}

	if ((LaneDescIndex + 1) < NumLanes)
	{
		const FZoneLaneDesc& NextLaneDesc = LaneDescs[LaneDescIndex + 1];
		if (NextLaneDesc.Direction != EZoneLaneDirection::None)
		{
			if (LaneDesc.Direction != NextLaneDesc.Direction)
			{
				NextLinkFlags |= EZoneLaneLinkFlags::OppositeDirection;
			}
			OutInternalLinks.Emplace(CurrentLaneIndex, FZoneLaneLinkData(CurrentLaneIndex + 1, EZoneLaneLinkType::Adjacent, NextLinkFlags));
		}
	}

	if ((LaneDescIndex - 1) >= 0)
	{
		const FZoneLaneDesc& PrevLaneDesc = LaneDescs[LaneDescIndex - 1];
		if (PrevLaneDesc.Direction != EZoneLaneDirection::None)
		{
			if (LaneDesc.Direction != PrevLaneDesc.Direction)
			{
				PrevLinkFlags |= EZoneLaneLinkFlags::OppositeDirection;
			}
			OutInternalLinks.Emplace(CurrentLaneIndex, FZoneLaneLinkData(CurrentLaneIndex - 1, EZoneLaneLinkType::Adjacent, PrevLinkFlags));
		}
	}
}

void TessellateSplineShape(TConstArrayView<FZoneShapePoint> Points, const FZoneLaneProfile& LaneProfile, const FZoneGraphTagMask ZoneTags, const FMatrix& LocalToWorld,
							FZoneGraphStorage& OutZoneStorage, TArray<FZoneShapeLaneInternalLink>& OutInternalLinks)
{
	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
	check(ZoneGraphSettings);
	const FZoneGraphBuildSettings& BuildSettings = ZoneGraphSettings->GetBuildSettings();

	const bool bClosedShape = false;

	const int32 ZoneIndex = OutZoneStorage.Zones.Num();
	FZoneData& Zone = OutZoneStorage.Zones.AddDefaulted_GetRef();
	Zone.Tags = ZoneTags;

	const float TessTolerance = GetMinLaneProfileTessellationTolerance(LaneProfile, ZoneTags, BuildSettings);

	// Flatten spline segments to points.
	TArray<FShapePoint> CurvePoints;
	FlattenSplineSegments(Points, bClosedShape, LocalToWorld, TessTolerance, CurvePoints);

	// Remove points which are too close to each other.
	RemoveDegenerateSegments(CurvePoints, bClosedShape);

	// Calculate edge normals.
	TArray<FVector> EdgeNormals;
	CalculateEdgeNormals(CurvePoints, EdgeNormals);

	CalculateStartAndEndNormals(Points, LocalToWorld, EdgeNormals[0], EdgeNormals[EdgeNormals.Num() - 1]);

	// Calculate miter extrusion at vertices
	CalculateMiters(CurvePoints, EdgeNormals);

	// Build spline boundary polygon
	Zone.BoundaryPointsBegin = OutZoneStorage.BoundaryPoints.Num();
	const float TotalWidth = LaneProfile.GetLanesTotalWidth();
	const float HalfWidth = TotalWidth * 0.5f;
	for (int32 i = 0; i < CurvePoints.Num(); i++)
	{
		const FShapePoint& Point = CurvePoints[i];
		OutZoneStorage.BoundaryPoints.Add(Point.Position - Point.Right * HalfWidth);
	}
	for (int32 i = CurvePoints.Num() - 1; i >= 0; i--)
	{
		const FShapePoint& Point = CurvePoints[i];
		OutZoneStorage.BoundaryPoints.Add(Point.Position + Point.Right * HalfWidth);
	}
	Zone.BoundaryPointsEnd = OutZoneStorage.BoundaryPoints.Num();

	// Build lanes

	const FVector StartForward = LocalToWorld.TransformVector(Points[0].Rotation.RotateVector(FVector::ForwardVector).GetSafeNormal());
	const FVector EndForward = LocalToWorld.TransformVector(Points.Last().Rotation.RotateVector(FVector::ForwardVector).GetSafeNormal());

	TArray<FShapePoint> LanePoints;
	Zone.LanesBegin = OutZoneStorage.Lanes.Num();
	const int32 NumLanes = LaneProfile.Lanes.Num();

	const uint16 FirstPointID = 0;
	const uint16 LastPointID = uint16(Points.Num() - 1);
	
	float CurWidth = 0.0f;
	for (int32 i = 0; i < NumLanes; i++)
	{
		const FZoneLaneDesc& LaneDesc = LaneProfile.Lanes[i];

		// Skip spacer lanes, but apply their width.
		if (LaneDesc.Direction == EZoneLaneDirection::None)
		{
			CurWidth += LaneDesc.Width;
			continue;
		}

		FZoneLaneData& Lane = OutZoneStorage.Lanes.AddDefaulted_GetRef();
		Lane.ZoneIndex = ZoneIndex;
		Lane.Width = LaneDesc.Width;
		Lane.Tags = LaneDesc.Tags | ZoneTags;
		// Store which inputs points corresponds to the lane start/end point.
		Lane.StartEntryId = FirstPointID;
		Lane.EndEntryId = LastPointID;
		const int32 CurrentLaneIndex = OutZoneStorage.Lanes.Num() - 1;

		// Add internal adjacent links.
		AddAdjacentLaneLinks(CurrentLaneIndex, i, LaneProfile.Lanes, OutInternalLinks);

		const float LanePos = HalfWidth - (CurWidth + LaneDesc.Width * 0.5f);

		// Create lane points.
		LanePoints.Reset();
		if (LaneDesc.Direction == EZoneLaneDirection::Forward)
		{
			for (int32 j = 0; j < CurvePoints.Num(); j++)
			{
				const FShapePoint& Point = CurvePoints[j];
				FShapePoint& NewPoint = LanePoints.Add_GetRef(Point);
				NewPoint.Position += Point.Right * LanePos;
			}
		}
		else if (LaneDesc.Direction == EZoneLaneDirection::Backward)
		{
			Swap(Lane.StartEntryId, Lane.EndEntryId);

			for (int32 j = CurvePoints.Num() - 1; j >= 0; j--)
			{
				const FShapePoint& Point = CurvePoints[j];
				FShapePoint& NewPoint = LanePoints.Add_GetRef(Point);
				NewPoint.Position += Point.Right * LanePos;
			}
		}
		else
		{
			ensure(false);
		}

		// The spline is tessellated at the finest level of all lanes, simplify it to match the lanes tessellation tolerance.
		const float LaneTessTolerance = BuildSettings.GetLaneTessellationTolerance(Lane.Tags);
		SimplifyShape(LanePoints, LaneTessTolerance);

		Lane.PointsBegin = OutZoneStorage.LanePoints.Num();
		for (const FShapePoint& Point : LanePoints)
		{
			OutZoneStorage.LanePoints.Add(Point.Position);
			OutZoneStorage.LaneUpVectors.Add(Point.Up);
		}
		Lane.PointsEnd = OutZoneStorage.LanePoints.Num();

		// Calculate per point forward.
		if (LaneDesc.Direction == EZoneLaneDirection::Forward)
		{
			OutZoneStorage.LaneTangentVectors.Add(StartForward);
		}
		else
		{
			OutZoneStorage.LaneTangentVectors.Add(-EndForward);
		}
		
		for (int32 PointIndex = Lane.PointsBegin + 1; PointIndex < Lane.PointsEnd - 1; PointIndex++)
		{
			const FVector WorldTangent = (OutZoneStorage.LanePoints[PointIndex + 1] - OutZoneStorage.LanePoints[PointIndex - 1]).GetSafeNormal();
			OutZoneStorage.LaneTangentVectors.Add(WorldTangent);
		}

		if (LaneDesc.Direction == EZoneLaneDirection::Forward)
		{
			OutZoneStorage.LaneTangentVectors.Add(EndForward);
		}
		else
		{
			OutZoneStorage.LaneTangentVectors.Add(-StartForward);
		}


		CurWidth += LaneDesc.Width;
	}

	Zone.LanesEnd = OutZoneStorage.Lanes.Num();

	// Calculate progression distance along lanes.
	OutZoneStorage.LanePointProgressions.AddZeroed(OutZoneStorage.LanePoints.Num() - OutZoneStorage.LanePointProgressions.Num());
	for (int32 i = Zone.LanesBegin; i < Zone.LanesEnd; i++)
	{
		const FZoneLaneData& Lane = OutZoneStorage.Lanes[i];
		CalculateLaneProgression(OutZoneStorage.LanePoints, Lane.PointsBegin, Lane.PointsEnd, OutZoneStorage.LanePointProgressions);
	}

	// Calculate zone bounding box, all lanes are assumed to be inside the boundary
	Zone.Bounds.Init();
	for (int32 i = Zone.BoundaryPointsBegin; i < Zone.BoundaryPointsEnd; i++)
	{
		Zone.Bounds += OutZoneStorage.BoundaryPoints[i];
	}
}


struct FLaneConnectionSlot
{
	FVector Position = FVector::ZeroVector;
	FVector Forward = FVector::ZeroVector;
	FVector Up = FVector::ZeroVector;
	FZoneLaneDesc LaneDesc;
	int32 PointIndex = 0;	// Index in dest point array
	int32 Index = 0;		// Index within an entry
	uint16 EntryID = 0;		// Entry ID from source data
	const FZoneLaneProfile* Profile = nullptr;
	EZoneShapeLaneConnectionRestrictions Restrictions = EZoneShapeLaneConnectionRestrictions::None;
	float DistanceFromProfileEdge = 0.0f;	// Distance from lane profile edge
	float DistanceFromFarProfileEdge = 0.0f; // Distance to other lane profile edge
	float InnerTurningRadius = 0.0f; // Inner/minimum turning radius when using Arc routing.
};

struct FLaneConnectionCandidate
{
	FLaneConnectionCandidate() = default;
	FLaneConnectionCandidate(const int32 InSourceSlot, const int32 InDestSlot, const FZoneGraphTagMask InTagMask) : SourceSlot(InSourceSlot), DestSlot(InDestSlot), TagMask(InTagMask) {}
	int32 SourceSlot = 0;
	int32 DestSlot = 0;
	FZoneGraphTagMask TagMask;
};

static bool CalcDestinationSide(TConstArrayView<FLaneConnectionSlot> SourceSlots, TConstArrayView<FLaneConnectionSlot> DestSlots)
{
	const FVector SourceCenter = (SourceSlots[0].Position + SourceSlots.Last().Position);
	const FVector DestCenter = (DestSlots[0].Position + DestSlots.Last().Position);
	const FVector Forward = (SourceSlots[0].Forward - DestSlots[0].Forward).GetSafeNormal();
	const FVector SourceSide = FVector::CrossProduct(Forward, SourceSlots[0].Up);
	const float ProjectedPos = FVector::DotProduct(SourceSide, DestCenter - SourceCenter);
	return ProjectedPos < 0.0f;
}

static int32 FitRange(const int32 RangeFirst, const int32 RangeNum, const int32 Space)
{
	int32 First = FMath::Max(0, RangeFirst);
	if ((First + RangeNum) > Space)
	{
		First = Space - RangeNum;
	}
	return First;
}

static void AddOrUpdateConnection(TArray<FLaneConnectionCandidate>& Candidates, const int32 SourceSlot, const int32 DestSlot, const FZoneGraphTag Tag)
{
	FLaneConnectionCandidate* Cand = Candidates.FindByPredicate([SourceSlot, DestSlot](const FLaneConnectionCandidate& Cand) -> bool { return Cand.SourceSlot == SourceSlot&& Cand.DestSlot == DestSlot; });
	if (Cand != nullptr)
	{
		Cand->TagMask = Cand->TagMask | FZoneGraphTagMask(Tag);
	}
	else
	{
		Candidates.Add(FLaneConnectionCandidate(SourceSlot, DestSlot, FZoneGraphTagMask(Tag)));
	}
}
	
static void AppendLaneConnectionCandidates(TArray<FLaneConnectionCandidate>& Candidates, TConstArrayView<FLaneConnectionSlot> SourceSlots, TConstArrayView<FLaneConnectionSlot> DestSlots,
										   const FZoneGraphTag Tag, const int32 MainDestPointIndex)
{
	const int32 SourceNum = SourceSlots.Num();
	const int32 DestNum = DestSlots.Num();

	if (SourceNum == 0 || DestNum == 0)
	{
		return;
	}

	// Expect connection between slots on two entries (polygon points), PointIndex and Restrictions should be same for all slots.
	const int32 DestPointIndex = DestSlots[0].PointIndex;
	const EZoneShapeLaneConnectionRestrictions ConnectionRestrictions = DestSlots[0].Restrictions;

	// We allow all lanes for the main connection, but just one lane for the side connections. 
	if (EnumHasAnyFlags(ConnectionRestrictions, EZoneShapeLaneConnectionRestrictions::OneLanePerDestination)
		&& DestPointIndex != MainDestPointIndex)
	{
		// Connect first or last lane to all destination lanes.
		const int32 SourceIdx = CalcDestinationSide(DestSlots, SourceSlots) ? 0 : (SourceNum - 1);
		const int32 DestIdx = CalcDestinationSide(SourceSlots, DestSlots) ? (DestNum - 1) : 0;

		// If a connection exists, we'll just update the tags, otherwise create new.
		AddOrUpdateConnection(Candidates, SourceSlots[SourceIdx].Index, DestSlots[DestIdx].Index, Tag);

		return;
	}
	
	if (EnumHasAnyFlags(ConnectionRestrictions, EZoneShapeLaneConnectionRestrictions::MergeLanesToOneDestinationLane))
	{
		// Fan in all the lanes to single lane in destination.
		const int32 BestDestLaneIndex = CalcDestinationSide(SourceSlots, DestSlots) ? (DestNum - 1) : 0;

		for (int32 i = 0; i < SourceNum; i++)
		{
			const int32 SourceIdx = i;
			const int32 DestIdx = BestDestLaneIndex;

			AddOrUpdateConnection(Candidates, SourceSlots[SourceIdx].Index, DestSlots[DestIdx].Index, Tag);
		}
		
		return;
	}

	const bool bOneLanePerDestination = EnumHasAnyFlags(ConnectionRestrictions, EZoneShapeLaneConnectionRestrictions::OneLanePerDestination);

	if (SourceNum < DestNum)
	{
		const int32 BestLaneIndex = CalcDestinationSide(SourceSlots, DestSlots) ? (DestNum - 1) : 0;

		// Distribute the lanes symmetrically around that best lane.
		const int32 FirstIndex = FitRange(BestLaneIndex - SourceNum/2, SourceNum, DestNum);

		for (int32 i = 0; i < DestNum; i++)
		{
			const int32 SourceIdxUnClamped = i - FirstIndex;
			if (bOneLanePerDestination && (SourceIdxUnClamped < 0 || SourceIdxUnClamped >= SourceNum))
			{
				continue;
			}

			const int32 SourceIdx = FMath::Clamp(SourceIdxUnClamped, 0, SourceNum - 1);
			const int32 DestIdx = i;
			
			AddOrUpdateConnection(Candidates, SourceSlots[SourceIdx].Index, DestSlots[DestIdx].Index, Tag);
		}
	}
	else
	{
		const int32 BestLaneIndex = CalcDestinationSide(SourceSlots, DestSlots) ? 0 : (SourceNum - 1);

		// Distribute the lanes symmetrically around that best lane.
		const int32 FirstIndex = FitRange(BestLaneIndex - DestNum/2, DestNum, SourceNum);
		
		for (int32 i = 0; i < SourceNum; i++)
		{
			const int32 SourceIdx = i;
			const int32 DestIdx = FMath::Clamp(i - FirstIndex, 0, DestNum - 1);
			
			AddOrUpdateConnection(Candidates, SourceSlots[SourceIdx].Index, DestSlots[DestIdx].Index, Tag);
		}
	}
}

static float GetLaneCompatibilityScore(const FZoneLaneProfile& ProfileA, const FZoneLaneProfile& ProfileB)
{
	// Profiles are the same, return full score.
	if (ProfileA.ID == ProfileB.ID)
	{
		return 1.f;
	}
	// Simple score based on number of lanes.
	const float MaxLaneNumScore = 0.75f;
	const int32 NumLanesA = ProfileA.Lanes.Num();
	const int32 NumLanesB = ProfileB.Lanes.Num();
	if (NumLanesA == NumLanesB == 0)
	{
		return MaxLaneNumScore;
	}
		
	const int32 MinLanes = FMath::Min(NumLanesA, NumLanesB);
	const int32 MaxLanes = FMath::Max(NumLanesA, NumLanesB);
	
	return MaxLaneNumScore * (MinLanes / (float)MaxLanes);
}

static bool CanConnectProfiles(const FZoneLaneProfile& SourceProfile, const FZoneLaneProfile& DestProfile)
{
	int32 PotentialConnections = 0;
	// Find how many lanes can be connected from source profile to dest profile based in direction and tags.
	for (int32 i = 0; i < SourceProfile.Lanes.Num(); i++)
	{
		const FZoneLaneDesc& SourceLaneDesc = SourceProfile.Lanes[i];
		if (SourceLaneDesc.Direction == EZoneLaneDirection::Forward)
		{
			for (int32 j = 0; j < DestProfile.Lanes.Num(); j++)
			{
				const FZoneLaneDesc& DestLaneDesc = DestProfile.Lanes[j];
				if (DestLaneDesc.Direction == EZoneLaneDirection::Backward && SourceLaneDesc.Tags.ContainsAny(DestLaneDesc.Tags))
				{
					PotentialConnections++;
				}
			}
		}
	}

	return PotentialConnections > 0;
}

struct FConnectionEntry
{
	FConnectionEntry(const FZoneShapePoint& InPoint, const FZoneLaneProfile& InProfile, const uint16 InEntryID, const int32 InOutgoingConnections, const int32 InIncomingConnections)
		: Point(InPoint), Profile(InProfile), EntryID(InEntryID), OutgoingConnections(InOutgoingConnections), IncomingConnections(InIncomingConnections)
	{}
	
	const FZoneShapePoint& Point;
	const FZoneLaneProfile& Profile;
	const uint16 EntryID;
	const int32 OutgoingConnections;
	const int32 IncomingConnections;
};

static void BuildLanesBetweenPoints(const FConnectionEntry& Source, TConstArrayView<FConnectionEntry> Destinations,
									const EZoneShapePolygonRoutingType RoutingType, const FZoneGraphTagMask ZoneTags, const FZoneGraphBuildSettings& BuildSettings, const FMatrix& LocalToWorld,
									FZoneGraphStorage& OutZoneStorage, TArray<FZoneShapeLaneInternalLink>& OutInternalLinks)
{
	const float LaneConnectionAngle = BuildSettings.LaneConnectionAngle;
	const float MaxConnectionAngleCos = FMath::Cos(FMath::DegreesToRadians(LaneConnectionAngle));
	const float TurnThresholdAngleCos = FMath::Cos(FMath::DegreesToRadians(BuildSettings.TurnThresholdAngle));

	const float SourceTotalWidth = Source.Profile.GetLanesTotalWidth();

	// Transform the lane segments into world space, and calculate forward vectors pointing into the shape.
	const FVector SourcePosition = LocalToWorld.TransformPosition(Source.Point.Position);
	const FVector SourceLaneLeft = LocalToWorld.TransformPosition(Source.Point.GetLaneProfileLeft());
	const FVector SourceLaneRight = LocalToWorld.TransformPosition(Source.Point.GetLaneProfileRight());
	const FVector SourceForward = LocalToWorld.TransformVector(Source.Point.Rotation.RotateVector(FVector::ForwardVector));
	const FVector SourceUp = LocalToWorld.TransformVector(Source.Point.Rotation.RotateVector(FVector::UpVector));

	FZoneGraphTagMask UsedTags = FZoneGraphTagMask::None;

	// Calculate potential source slots
	TArray<FLaneConnectionSlot> SourceSlots;

	float SourceWidth = 0.0f;
	for (int32 i = 0; i < Source.Profile.Lanes.Num(); i++)
	{
		const FZoneLaneDesc& SourceLaneDesc = Source.Profile.Lanes[i];

		// Connect only potentially outgoing lanes
		if (SourceLaneDesc.Direction == EZoneLaneDirection::Forward)
		{
			// Calculates position of the lane on the source segment.
			const float SourcePositionFrac = (SourceWidth + SourceLaneDesc.Width * 0.5f) / SourceTotalWidth;
			const FVector SourceLanePosition = FMath::Lerp(SourceLaneRight, SourceLaneLeft, SourcePositionFrac);

			FLaneConnectionSlot& Slot = SourceSlots.AddDefaulted_GetRef();
			Slot.Position = SourceLanePosition;
			Slot.Forward = SourceForward;
			Slot.Up = SourceUp;
			Slot.LaneDesc = SourceLaneDesc;
			Slot.Index = SourceSlots.Num() - 1;
			Slot.EntryID = Source.EntryID;
			Slot.Profile = &Source.Profile;
			Slot.DistanceFromProfileEdge = FMath::Lerp(0.0f, SourceTotalWidth, SourcePositionFrac);
			Slot.DistanceFromFarProfileEdge = SourceTotalWidth - Slot.DistanceFromProfileEdge;
			Slot.InnerTurningRadius = Source.Point.InnerTurnRadius;

			UsedTags = UsedTags | SourceLaneDesc.Tags;
		}
		SourceWidth += SourceLaneDesc.Width;
	}

	// Calculate potential destination slots.
	// Destinations are reversed so that source and destination arrays run in same direction.
	TArray<FLaneConnectionSlot> DestSlots;
	TArray<int32> DestSlotRanges;	// Slot ranges per point

	int32 MainDestPointIndex = 0;
	float MainDestScore = 0;
	
	for (int32 PointIndex = Destinations.Num() - 1; PointIndex >= 0; PointIndex--)
	{
		const FConnectionEntry& Dest = Destinations[PointIndex];
		const FZoneShapePoint& DestPoint = Dest.Point;
		const FZoneLaneProfile& DestLaneProfile = Dest.Profile;

		const float DestTotalWidth = DestLaneProfile.GetLanesTotalWidth();

		const FVector DestLaneLeft = LocalToWorld.TransformPosition(DestPoint.GetLaneProfileLeft());
		const FVector DestLaneRight = LocalToWorld.TransformPosition(DestPoint.GetLaneProfileRight());
		const FVector DestForward = LocalToWorld.TransformVector(DestPoint.Rotation.RotateVector(FVector::ForwardVector));
		const FVector DestUp = LocalToWorld.TransformVector(DestPoint.Rotation.RotateVector(FVector::UpVector));

		// Prune connections based on extreme angles.
		if (FVector::DotProduct(SourceForward, -DestForward) < MaxConnectionAngleCos)
		{
			continue;
		}

		const FVector ClosestDestPoint = FMath::ClosestPointOnSegment(SourcePosition, DestLaneLeft, DestLaneRight);
		const FVector DirToDest = (ClosestDestPoint - SourcePosition).GetSafeNormal();

		const bool bHasOneDestination = Destinations.Num() == 1;

		const EZoneShapeLaneConnectionRestrictions RestrictionsFromRules = BuildSettings.GetConnectionRestrictions(ZoneTags, Source.Profile, Source.OutgoingConnections,
																									Dest.Profile, Dest.IncomingConnections);

		const EZoneShapeLaneConnectionRestrictions Restrictions = Source.Point.GetLaneConnectionRestrictions() | RestrictionsFromRules;
		
		// Discard destination that would result in left or right turns.
		if (EnumHasAnyFlags(Restrictions, EZoneShapeLaneConnectionRestrictions::NoLeftTurn | EZoneShapeLaneConnectionRestrictions::NoRightTurn))
		{
			const bool bRemoveLeft = EnumHasAnyFlags(Restrictions, EZoneShapeLaneConnectionRestrictions::NoLeftTurn);
			const bool bRemoveRight = EnumHasAnyFlags(Restrictions, EZoneShapeLaneConnectionRestrictions::NoRightTurn);

			// Use closest point on dest so that lane profile widths do not affect direction.
			const FVector SourceSide = FVector::CrossProduct(SourceForward, SourceUp);
			
			const bool bIsTurning = FVector::DotProduct(SourceForward, DirToDest) < TurnThresholdAngleCos;
			const bool bIsLeftTurn = FVector::DotProduct(SourceSide, DirToDest) > 0.0f;
			if (bIsTurning)
			{
				const bool bSkip = bIsLeftTurn ? bRemoveLeft : bRemoveRight;
				if (bSkip)
				{
					continue;
				}
			}
		}

		// Keep track of the main connection.
		float Score = GetLaneCompatibilityScore(Source.Profile, Dest.Profile);
		if (FMath::IsNearlyEqual(Score, MainDestScore))
		{
			Score = FVector::DotProduct(SourceForward, DirToDest);
		}
		if (Score > MainDestScore)
		{
			MainDestScore = Score;
			MainDestPointIndex = PointIndex;
		}
		
		DestSlotRanges.Add(DestSlots.Num());

		float DestWidth = DestTotalWidth;
		for (int32 i = DestLaneProfile.Lanes.Num() - 1; i >= 0; i--)
		{
			const FZoneLaneDesc& DestLaneDesc = DestLaneProfile.Lanes[i];

			DestWidth -= DestLaneDesc.Width; // Decrement before use so that DestWidth has same meaning as in the source loop above.

			if (DestLaneDesc.Direction == EZoneLaneDirection::Backward)
			{
				// Calculates position of the lane on the destination segment.
				const float DestPositionFrac = (DestWidth + DestLaneDesc.Width * 0.5f) / DestTotalWidth;
				const FVector DestLanePosition = FMath::Lerp(DestLaneRight, DestLaneLeft, DestPositionFrac);

				FLaneConnectionSlot& Slot = DestSlots.AddDefaulted_GetRef();
				Slot.Position = DestLanePosition;
				Slot.Forward = DestForward;
				Slot.Up = DestUp;
				Slot.LaneDesc = DestLaneDesc;
				Slot.Index = DestSlots.Num() - 1;
				Slot.PointIndex = PointIndex;
				Slot.EntryID = Dest.EntryID;
				Slot.Profile = &DestLaneProfile;
				Slot.Restrictions = Restrictions;
				Slot.DistanceFromProfileEdge = FMath::Lerp(0.0f, DestTotalWidth, DestPositionFrac);
				Slot.DistanceFromFarProfileEdge = DestTotalWidth - Slot.DistanceFromProfileEdge;
				Slot.InnerTurningRadius = Dest.Point.InnerTurnRadius;

				UsedTags = UsedTags | DestLaneDesc.Tags;
			}
		}
	}

	DestSlotRanges.Add(DestSlots.Num());

	// Tags that are relevant to connecting lanes.
	UsedTags = UsedTags & BuildSettings.LaneConnectionMask;

	// Connect lanes the source to each destination at a time.
	TArray<FLaneConnectionSlot> TagSourceSlots;
	TArray<FLaneConnectionSlot> TagDestSlots;
	TArray<EZoneShapeLaneConnectionRestrictions> ConnectionRestrictions;
	TArray<FLaneConnectionCandidate> Candidates;

	for (int32 i = 0; i < DestSlotRanges.Num() - 1; i++)
	{
		const int32 DestSlotsBegin = DestSlotRanges[i];
		const int32 DestSlotsEnd = DestSlotRanges[i + 1];
		for (uint8 TagIndex = 0; TagIndex < uint8(EZoneGraphTags::MaxTags); TagIndex++)
		{
			FZoneGraphTag Tag = FZoneGraphTag(TagIndex);
			if (!UsedTags.Contains(Tag))
			{
				continue;
			}

			// Collect slots that have the current flag set.
			TagSourceSlots.Reset();
			for (const FLaneConnectionSlot& Slot : SourceSlots)
			{
				if (Slot.LaneDesc.Tags.Contains(Tag))
				{
					TagSourceSlots.Add(Slot);
				}
			}
			TagDestSlots.Reset();
			ConnectionRestrictions.Reset();
			for (int32 j = DestSlotsBegin; j < DestSlotsEnd; j++)
			{
				const FLaneConnectionSlot& Slot = DestSlots[j];
				if (Slot.LaneDesc.Tags.Contains(Tag))
				{
					const FConnectionEntry& Dest = Destinations[Slot.PointIndex];
					TagDestSlots.Add(Slot);
				}
			}

			if (TagSourceSlots.Num() > 0 && TagDestSlots.Num() > 0)
			{
				AppendLaneConnectionCandidates(Candidates, TagSourceSlots, TagDestSlots, Tag, MainDestPointIndex);
			}
		}
	}

	// Remove overlapping lanes.
	// AppendLaneConnectionCandidates() sees only source and one destination at a time.
	// This code removes any overlapping lanes and handles cases such as 4-lane entry might connect to two 2-lane exits.
	if (UE::ZoneGraph::Debug::bRemoveOverlap)
	{
		for (int32 Index = 0; Index < Candidates.Num() - 1; Index++)
		{
			const FLaneConnectionCandidate& Cand = Candidates[Index];
			for (int32 NextIndex = Index + 1; NextIndex < Candidates.Num(); NextIndex++)
			{
				const FLaneConnectionCandidate& NextCand = Candidates[NextIndex];
				if ((Cand.SourceSlot < NextCand.SourceSlot && Cand.DestSlot > NextCand.DestSlot) ||
					(Cand.SourceSlot > NextCand.SourceSlot && Cand.DestSlot < NextCand.DestSlot))
				{
					const FLaneConnectionSlot& SourceSlot = SourceSlots[Cand.SourceSlot];
					const FLaneConnectionSlot& DestSlot = DestSlots[Cand.DestSlot];
					const FLaneConnectionSlot& NextSourceSlot = SourceSlots[NextCand.SourceSlot];
					const FLaneConnectionSlot& NextDestSlot = DestSlots[NextCand.DestSlot];

					float Score = GetLaneCompatibilityScore(*SourceSlot.Profile, *DestSlot.Profile);
					float NextScore = GetLaneCompatibilityScore(*NextSourceSlot.Profile, *NextDestSlot.Profile);

					if (FMath::IsNearlyEqual(Score, NextScore))
					{
						const FVector Dir = (DestSlot.Position - SourceSlot.Position).GetSafeNormal();
						const FVector NextDir = (NextDestSlot.Position - NextSourceSlot.Position).GetSafeNormal();
						Score = FVector::DotProduct(SourceSlot.Forward, Dir);
						NextScore = FVector::DotProduct(NextSourceSlot.Forward, NextDir);
					}
				
					// Keep the link that is more straight.
					if (Score > NextScore)
					{
						Candidates.RemoveAt(NextIndex);
					}
					else
					{
						Candidates.RemoveAt(Index);
					}
				
					Index--;
					break;
				}
			}
		}
	}

	// Remove lanes that that connect to same destination as other lanes
	// This reduces the number of merging lanes when there are multiple destinations.
	if (UE::ZoneGraph::Debug::bRemoveSameDestination)
	{
		TArray<int32> SourceConnectionCount;
		TArray<int32> DestConnectionCount;
		SourceConnectionCount.SetNumZeroed(SourceSlots.Num());
		DestConnectionCount.SetNumZeroed(DestSlots.Num());
		for (const FLaneConnectionCandidate& Cand : Candidates)
		{
			SourceConnectionCount[Cand.SourceSlot]++;
			DestConnectionCount[Cand.DestSlot]++;
		}

		for (int32 Index = 0; Index < Candidates.Num() - 1; Index++)
		{
			const FLaneConnectionCandidate& Cand = Candidates[Index];
			for (int32 NextIndex = Index + 1; NextIndex < Candidates.Num(); NextIndex++)
			{
				const FLaneConnectionCandidate& NextCand = Candidates[NextIndex];

				if ((SourceConnectionCount[Cand.SourceSlot] == 1 && SourceConnectionCount[NextCand.SourceSlot] == 1)
					|| (DestConnectionCount[Cand.DestSlot] == 1 && DestConnectionCount[NextCand.DestSlot] == 1))
				{
					// Both source slots have only connection, do not remove the last one.
					continue;
				}
				
				// Remove lanes to same destination
				if (Cand.DestSlot == NextCand.DestSlot)
				{
					if (SourceConnectionCount[Cand.SourceSlot] == 1 && SourceConnectionCount[NextCand.SourceSlot] > 1)
					{
						// "Cand" cannot be removed since it would leave source unconnected.
						SourceConnectionCount[NextCand.SourceSlot]--;
						DestConnectionCount[NextCand.DestSlot]--;
						Candidates.RemoveAt(NextIndex);
					}
					else if (SourceConnectionCount[NextCand.SourceSlot] == 1 && SourceConnectionCount[Cand.SourceSlot] > 1)
					{
						// "NextCand" cannot be removed since it would leave source unconnected.
						SourceConnectionCount[Cand.SourceSlot]--;
						DestConnectionCount[Cand.DestSlot]--;
						Candidates.RemoveAt(Index);
					}
					else if (DestConnectionCount[Cand.DestSlot] == 1 && DestConnectionCount[NextCand.DestSlot] > 1)
					{
						// "Cand" cannot be removed since it would leave dest unconnected.
						SourceConnectionCount[NextCand.SourceSlot]--;
						DestConnectionCount[NextCand.DestSlot]--;
						Candidates.RemoveAt(NextIndex);
					}
					else if (DestConnectionCount[NextCand.DestSlot] == 1 && DestConnectionCount[Cand.DestSlot] > 1)
					{
						// "NextCand" cannot be removed since it would leave dest unconnected.
						SourceConnectionCount[Cand.SourceSlot]--;
						DestConnectionCount[Cand.DestSlot]--;
						Candidates.RemoveAt(Index);
					}
					else
					{
						// Both connections can be removed, keep the link that is more straight.
						const FLaneConnectionSlot& SourceSlot = SourceSlots[Cand.SourceSlot];
						const FLaneConnectionSlot& DestSlot = DestSlots[Cand.DestSlot];
						const FLaneConnectionSlot& NextSourceSlot = SourceSlots[NextCand.SourceSlot];
						const FLaneConnectionSlot& NextDestSlot = DestSlots[NextCand.DestSlot];

						// Favor to connect destinations that share the same lane profile as the source, if profiles are about as good match, favor straight connections. 
						float Score = GetLaneCompatibilityScore(*SourceSlot.Profile, *DestSlot.Profile);
						float NextScore = GetLaneCompatibilityScore(*NextSourceSlot.Profile, *NextDestSlot.Profile);
						if (FMath::IsNearlyEqual(Score, NextScore))
						{
							const FVector Dir = (DestSlot.Position - SourceSlot.Position).GetSafeNormal();
							const FVector NextDir = (NextDestSlot.Position - NextSourceSlot.Position).GetSafeNormal();
							Score = FVector::DotProduct(SourceSlot.Forward, Dir);
							NextScore = FVector::DotProduct(NextSourceSlot.Forward, NextDir);
						}

						if (Score > NextScore)
						{
							SourceConnectionCount[NextCand.SourceSlot]--;
							DestConnectionCount[NextCand.DestSlot]--;
							Candidates.RemoveAt(NextIndex);
						}
						else
						{
							SourceConnectionCount[Cand.SourceSlot]--;
							DestConnectionCount[Cand.DestSlot]--;
							Candidates.RemoveAt(Index);
						}
					}
					Index--;
					break;
				}
			}
		}
	}

	
	if (UE::ZoneGraph::Debug::bFillEmptyDestination)
	{
		// Fill in empty destination connections if possible.
		// This usually happens when overlaps are removed, and i can leave left or right turn destinations empty.
		// In that case this code will duplicate near connections to connect the empty lanes.
		TArray<int32> DestConnectionCount;
		DestConnectionCount.SetNumZeroed(DestSlots.Num());
		for (const FLaneConnectionCandidate& Cand : Candidates)
		{
			DestConnectionCount[Cand.DestSlot]++;
		}

		for (int32 DestSlotIdx = 0; DestSlotIdx < DestConnectionCount.Num(); DestSlotIdx++)
		{
			// Skip slots that are already connected.
			if (DestConnectionCount[DestSlotIdx] > 0)
			{
				continue;
			}

			const FLaneConnectionSlot& DestSlot = DestSlots[DestSlotIdx];

			const bool bMergeLanesToOneDestinationLane = EnumHasAnyFlags(DestSlot.Restrictions, EZoneShapeLaneConnectionRestrictions::MergeLanesToOneDestinationLane);
			const bool bOneLanePerEntry = EnumHasAnyFlags(DestSlot.Restrictions, EZoneShapeLaneConnectionRestrictions::OneLanePerDestination);
			if (bMergeLanesToOneDestinationLane || bOneLanePerEntry)
			{
				continue;
			}

			// Find nearest candidate that points to the same point index.
			const int32 DestPointIndex = DestSlot.PointIndex;
			const FZoneGraphTagMask DestTags = DestSlot.LaneDesc.Tags;
			int32 NearestOffset = MAX_int32;
			int32 NearestSourceSlot = INDEX_NONE;
			FZoneGraphTagMask NearestSlotTags = FZoneGraphTagMask::None;
			for (const FLaneConnectionCandidate& Cand : Candidates)
			{
				if (DestSlots[Cand.DestSlot].PointIndex == DestPointIndex)
				{
					const FZoneGraphTagMask SourceTags = SourceSlots[Cand.SourceSlot].LaneDesc.Tags;
					if (SourceTags.ContainsAny(DestTags))
					{
						const int32 Offset = FMath::Abs(Cand.DestSlot - DestSlotIdx);
						if (Offset < NearestOffset)
						{
							NearestOffset = Offset;
							NearestSourceSlot = Cand.SourceSlot;
							NearestSlotTags = SourceTags & DestTags;
						}
					}
				}
			}
			if (NearestSourceSlot != INDEX_NONE)
			{
				Candidates.Add(FLaneConnectionCandidate(NearestSourceSlot, DestSlotIdx, NearestSlotTags));
			}
		}
	}

	// Sort candidates for lane adjacency. First by source index, then by destination index.
	// Lane adjacency is not that obvious in polygons. With this sort we make sure that they are somewhat in order and that the whole set can be iterated over.
	Candidates.Sort([](const FLaneConnectionCandidate& A, const FLaneConnectionCandidate& B) { return A.SourceSlot < B.SourceSlot || (A.SourceSlot == B.SourceSlot && A.DestSlot < B.DestSlot); });

	// Create lanes from candidates.
	const int32 NumLanes = Candidates.Num();
	for (int32 i = 0; i < NumLanes; i++)
	{
		const FLaneConnectionCandidate& Cand = Candidates[i];
		const FLaneConnectionSlot& SourceSlot = SourceSlots[Cand.SourceSlot];
		const FLaneConnectionSlot& DestSlot = DestSlots[Cand.DestSlot];

		// Add lane
		// Note: Lane adjacency for polygons are calculated in ZoneGraphBuilder.
		FZoneLaneData& Lane = OutZoneStorage.Lanes.AddDefaulted_GetRef();

		// Store which inputs points (indicated as entry ID) corresponds to the lane start/end point.
		Lane.StartEntryId = SourceSlot.EntryID;
		Lane.EndEntryId = DestSlot.EntryID;
		
		// Merge values from descriptors.
		Lane.Width = SourceSlot.LaneDesc.Width; // Copy over source width, source and dest lane widths are the same.

		// Separate the tags which are not part of the lane connection from the lane desc.
		FZoneGraphTagMask LaneTags = SourceSlot.LaneDesc.Tags & ~BuildSettings.LaneConnectionMask;
		// The candidate tags contains the tags that matched between source and target slot, merge with rest of the tags.
		Lane.Tags = Cand.TagMask | LaneTags | ZoneTags;

		Lane.PointsBegin = OutZoneStorage.LanePoints.Num();

		if (RoutingType == EZoneShapePolygonRoutingType::Bezier)
		{
			// Calculate Bezier curve connecting the source and destination.
			const float TangentLength = FVector::Distance(SourceSlot.Position, DestSlot.Position) / 3.0f;
			const FVector SourceControlPoint = SourceSlot.Position + SourceSlot.Forward * TangentLength;
			const FVector DestControlPoint = DestSlot.Position + DestSlot.Forward * TangentLength;

			OutZoneStorage.LanePoints.Add(SourceSlot.Position); // Explicitly add the start point as tessellate omits the start point.
			const float TessTolerance = BuildSettings.GetLaneTessellationTolerance(Lane.Tags);
			UE::CubicBezier::Tessellate(OutZoneStorage.LanePoints, SourceSlot.Position, SourceControlPoint, DestControlPoint, DestSlot.Position, TessTolerance);
		}
		else // EZoneShapePolygonRoutingType::Arcs
		{
			// TODO: To make this more generic, we could first find a plane and project the points on it, instead of assuming XY plane. Or, handle the trajectory in 3D.
			const float SourceRadiusCCW = SourceSlot.InnerTurningRadius + SourceSlot.DistanceFromProfileEdge;
			const float SourceRadiusCW = SourceSlot.InnerTurningRadius + SourceSlot.DistanceFromFarProfileEdge;
			const float DestRadiusCCW = DestSlot.InnerTurningRadius + DestSlot.DistanceFromFarProfileEdge;
            const float DestRadiusCW = DestSlot.InnerTurningRadius + DestSlot.DistanceFromProfileEdge;
			const float MinTurnRadius = FMath::Min(SourceSlot.InnerTurningRadius, DestSlot.InnerTurningRadius);

			// End direction is reversed, because the slot forward vector points into the polygon, but the path calculation assumes movement direction.
			FDubinsPath Path;
			CalculateIntersectingOrDubinsPath(FVector2D(SourceSlot.Position), FVector2D(SourceSlot.Forward), SourceRadiusCCW, SourceRadiusCW,
														FVector2D(DestSlot.Position), -FVector2D(DestSlot.Forward), DestRadiusCCW, DestRadiusCW,
														MinTurnRadius, Path);
		
			const float TessTolerance = BuildSettings.GetLaneTessellationTolerance(Lane.Tags);
			TessellateDubinsPath(Path, OutZoneStorage.LanePoints, TessTolerance);

			// Reconstruct height.
			// TODO: We could use i.e. bezier curve to more smoothly vary the height.
			const int32 NumPoints = OutZoneStorage.LanePoints.Num() - Lane.PointsBegin;
			ensureMsgf(NumPoints > 1, TEXT("Arc lanes must have more than one point for height reconstruction."));
			for (int32 PointIndex = 0; PointIndex < NumPoints; PointIndex++)
			{
				const float Alpha = (float)PointIndex / (float)(NumPoints - 1);
				OutZoneStorage.LanePoints[Lane.PointsBegin + PointIndex].Z = FMath::Lerp(SourceSlot.Position.Z, DestSlot.Position.Z, Alpha);
			}
		}
		
		Lane.PointsEnd = OutZoneStorage.LanePoints.Num();

		// TODO: Consider templated version of UE::CubicBezier::Tessellate() which could handle this.
		// Interpolate up vector for points
		const int32 NumNewPoints = Lane.PointsEnd - Lane.PointsBegin;
		TArrayView<FVector> NewPoints(&OutZoneStorage.LanePoints[Lane.PointsBegin], NumNewPoints);
		TArray<float> TempProgression;
		TempProgression.SetNum(NumNewPoints);

		// Interpolate up vector for points
		float TotalDist = 0;
		for (int32 PointIndex = 0; PointIndex < NumNewPoints - 1; PointIndex++)
		{
			TempProgression[PointIndex] = TotalDist;
			TotalDist += FVector::Dist(NewPoints[PointIndex], NewPoints[PointIndex + 1]);
		}
		TempProgression[TempProgression.Num() - 1] = TotalDist;

		// Interpolate up axis
		for (int32 PointIndex = 0; PointIndex < NumNewPoints; PointIndex++)
		{
			const float Alpha = TempProgression[PointIndex] / TotalDist;
			const FVector WorldUp = FMath::Lerp(SourceSlot.Up, DestSlot.Up, Alpha).GetSafeNormal(); // TODO: quat/spherical interpolation?
			OutZoneStorage.LaneUpVectors.Add(WorldUp);
		}

		// Calculate per point forward. 
		OutZoneStorage.LaneTangentVectors.Add(SourceSlot.Forward);
		for (int32 PointIndex = 1; PointIndex < NumNewPoints - 1; PointIndex++)
		{
			const FVector WorldTangent = (NewPoints[PointIndex + 1] - NewPoints[PointIndex - 1]).GetSafeNormal();
			OutZoneStorage.LaneTangentVectors.Add(WorldTangent);
		}
		OutZoneStorage.LaneTangentVectors.Add(-DestSlot.Forward);
	}
}

float GetPolygonBoundaryTessellationTolerance(TConstArrayView<FZoneLaneProfile> LaneProfiles, const FZoneGraphTagMask ZoneTags, const FZoneGraphBuildSettings& BuildSettings)
{
	float TessTolerance = MAX_flt;
	for (const FZoneLaneProfile& Profile : LaneProfiles)
	{
		const float ProfileTessToleranceAvg = GetMinLaneProfileTessellationTolerance(Profile, ZoneTags, BuildSettings);
		TessTolerance = FMath::Min(TessTolerance, ProfileTessToleranceAvg);
	}
	return TessTolerance;
}

void TessellatePolygonShape(TConstArrayView<FZoneShapePoint> Points, const EZoneShapePolygonRoutingType RoutingType, TConstArrayView<FZoneLaneProfile> LaneProfiles, const FZoneGraphTagMask ZoneTags, const FMatrix& LocalToWorld,
							FZoneGraphStorage& OutZoneStorage, TArray<FZoneShapeLaneInternalLink>& OutInternalLinks)
{
	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
	check(ZoneGraphSettings);
	const FZoneGraphBuildSettings& BuildSettings = ZoneGraphSettings->GetBuildSettings();

	const int32 ZoneIndex = OutZoneStorage.Zones.Num();
	FZoneData& Zone = OutZoneStorage.Zones.AddDefaulted_GetRef();
	Zone.Tags = ZoneTags;

	const float BoundaryTessTolerance = GetPolygonBoundaryTessellationTolerance(LaneProfiles, ZoneTags, BuildSettings);

	const bool bClosedShape = true;

	// Flatten spline segments to points.
	TArray<FShapePoint> CurvePoints;
	FlattenSplineSegments(Points, bClosedShape, LocalToWorld, BoundaryTessTolerance, CurvePoints);

	// Remove points which are too close to each other.
	RemoveDegenerateSegments(CurvePoints, bClosedShape);

	// Build boundary polygon
	Zone.BoundaryPointsBegin = OutZoneStorage.BoundaryPoints.Num();
	for (int32 i = 0; i < CurvePoints.Num(); i++)
	{
		OutZoneStorage.BoundaryPoints.Add(CurvePoints[i].Position);
	}
	Zone.BoundaryPointsEnd = OutZoneStorage.BoundaryPoints.Num();

	// Build lanes

	// Calculate over optimistic estimate how entried connect.
	// TODO: make this more accurate, maybe cache the first part of BuildLanesBetweenPoints() before calculating actual lanes.
	TArray<int32> OutgoingConnections;
	TArray<int32> IncomingConnections;
	OutgoingConnections.Init(0, Points.Num());
	IncomingConnections.Init(0, Points.Num());
	
	for (int32 SourceIdx = 0; SourceIdx < Points.Num(); SourceIdx++)
	{
		const FZoneShapePoint& SourcePoint = Points[SourceIdx];
		const FZoneLaneProfile& SourceLaneProfile = LaneProfiles[SourceIdx];
		if (SourcePoint.Type != FZoneShapePointType::LaneProfile)
		{
			continue;
		}
		for (int32 j = 1; j < Points.Num(); j++)
		{
			const int DestIdx = (SourceIdx + j) % Points.Num();
			const FZoneShapePoint& DestPoint = Points[DestIdx];
			const FZoneLaneProfile& DestLaneProfile = LaneProfiles[DestIdx];
			if (DestPoint.Type != FZoneShapePointType::LaneProfile)
			{
				continue;
			}

			if (CanConnectProfiles(SourceLaneProfile, DestLaneProfile))
			{
				OutgoingConnections[SourceIdx]++;
				IncomingConnections[DestIdx]++;
			}
		}
	}
	// Connect each lane vertex to another lane vertex
	Zone.LanesBegin = OutZoneStorage.Lanes.Num();

	TArray<FConnectionEntry> Destinations;
	check(Points.Num() <= (int32)MAX_uint16);
	for (uint16 SourceIdx = 0; SourceIdx < Points.Num(); SourceIdx++)
	{
		const FZoneShapePoint& SourcePoint = Points[SourceIdx];
		const FZoneLaneProfile& SourceLaneProfile = LaneProfiles[SourceIdx];
		if (SourcePoint.Type != FZoneShapePointType::LaneProfile)
		{
			continue;
		}
		// Collect all potential destinations
		Destinations.Reset();
		for (uint16 j = 1; j < Points.Num(); j++)
		{
			const int DestIdx = (SourceIdx + j) % Points.Num();
			const FZoneShapePoint& DestPoint = Points[DestIdx];
			const FZoneLaneProfile& DestLaneProfile = LaneProfiles[DestIdx];
			if (DestPoint.Type != FZoneShapePointType::LaneProfile)
			{
				continue;
			}

			Destinations.Emplace(DestPoint, DestLaneProfile, (uint16)DestIdx, OutgoingConnections[DestIdx], IncomingConnections[DestIdx]);
		}
		// Connect source to destinations.
		BuildLanesBetweenPoints(FConnectionEntry(SourcePoint, SourceLaneProfile, SourceIdx, OutgoingConnections[SourceIdx], IncomingConnections[SourceIdx]),
								Destinations, RoutingType, ZoneTags, BuildSettings, LocalToWorld, OutZoneStorage, OutInternalLinks);
	}

	Zone.LanesEnd = OutZoneStorage.Lanes.Num();

	// Apply zone index
	for (int32 i = Zone.LanesBegin; i < Zone.LanesEnd; i++)
	{
		OutZoneStorage.Lanes[i].ZoneIndex = ZoneIndex;
	}

	// Calculate progression distance along lanes.
	OutZoneStorage.LanePointProgressions.AddZeroed(OutZoneStorage.LanePoints.Num() - OutZoneStorage.LanePointProgressions.Num());
	for (int32 i = Zone.LanesBegin; i < Zone.LanesEnd; i++)
	{
		const FZoneLaneData& Lane = OutZoneStorage.Lanes[i];
		CalculateLaneProgression(OutZoneStorage.LanePoints, Lane.PointsBegin, Lane.PointsEnd, OutZoneStorage.LanePointProgressions);
	}

	// Calculate zone bounding box, all lanes are assumed to be inside the boundary
	Zone.Bounds.Init();
	for (int32 i = Zone.BoundaryPointsBegin; i < Zone.BoundaryPointsEnd; i++)
	{
		Zone.Bounds += OutZoneStorage.BoundaryPoints[i];
	}
}

} // UE::ZoneShape::Utilities
