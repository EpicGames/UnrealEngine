// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Deprecated

#include "Curves/BezierUtilities.h"

UE_DEPRECATED_HEADER(5.5, "BezierUtilities.h is deprecated; file moved to Curves/BezierUtilities.h.");
