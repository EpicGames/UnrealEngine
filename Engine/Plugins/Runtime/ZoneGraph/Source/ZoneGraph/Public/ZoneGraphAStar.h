// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GraphAStar.h"
#include "ZoneGraphTypes.h"
#include "ZoneGraphQuery.h"

#define UE_API ZONEGRAPH_API

struct FZoneGraphAStarNode;

/** Warpper around zone graph to be used by FGraphAStar */
struct FZoneGraphAStarWrapper
{
	FZoneGraphAStarWrapper(const FZoneGraphStorage& InZoneGraph) 
		: ZoneGraph(InZoneGraph) 
	{}

	//////////////////////////////////////////////////////////////////////////
	// FGraphAStar: TGraph
	typedef int32/*lane index type*/ FNodeRef;

	inline bool IsValidRef(const FNodeRef NodeRef) const
	{
		return NodeRef != INDEX_NONE;
	}

	UE_API int32 GetNeighbourCountV2(const FZoneGraphAStarNode& Node) const;
	UE_API FNodeRef GetNeighbour(const FZoneGraphAStarNode& Node, const int32 NeighbourIndex) const;
	//////////////////////////////////////////////////////////////////////////

protected:
	const FZoneGraphStorage& ZoneGraph;
};

/** Node representation for FZoneGraphAStar */
struct FZoneGraphAStarNode : public FGraphAStarDefaultNode<FZoneGraphAStarWrapper>
{
	typedef FGraphAStarDefaultNode<FZoneGraphAStarWrapper> Super;
	typedef int32/*lane index type*/ FNodeRef;

	inline FZoneGraphAStarNode(const FNodeRef InNodeRef = INDEX_NONE, const FVector InPosition = FVector(TNumericLimits<FVector::FReal>::Max()))
		: Super(InNodeRef)
		, Position(InPosition)
	{}

	FZoneGraphAStarNode(const FZoneGraphAStarNode& Other) = default;
	FGraphAStarDefaultNode& operator=(const FGraphAStarDefaultNode& Other) = delete;

	bool IsStartOrIsEnd() const { return Position != FVector(TNumericLimits<FVector::FReal>::Max()); };

	FVector Position; //@todo: this will likely change to be a position along the lane
};

/** Context for FGraphAStar::FindPath() */
struct FZoneGraphPathFilter
{
	// @todo: rename FZoneGraphPathfindContext?

	typedef int32/*lane index type*/ FNodeRef;

	FZoneGraphPathFilter(const FZoneGraphStorage& InGraph, const FZoneGraphLaneLocation& InStartLocation, const FZoneGraphLaneLocation& InEndLocation, const FZoneGraphTagFilter InZoneTagFilter = FZoneGraphTagFilter())
		: ZoneStorage(InGraph) 
		, ZoneTagFilter(InZoneTagFilter)
		, StartLocation(InStartLocation)
		, EndLocation(InEndLocation)
	{}

	bool IsStart(const FZoneGraphAStarNode& Node) const { return Node.NodeRef == StartLocation.LaneHandle.Index; }
	bool IsEnd(const FZoneGraphAStarNode& Node) const { return Node.NodeRef == EndLocation.LaneHandle.Index; }

	inline FVector::FReal GetHeuristicScale() const { return 1.; }

	FVector::FReal GetHeuristicCost(const FZoneGraphAStarNode& NeighbourNode, const FZoneGraphAStarNode& EndNode) const
	{
		static const FVector InvalidPosition(TNumericLimits<FVector::FReal>::Max());
		
		// Except for start to end nodes, compute heurisitic from the last point of lanes
		if (NeighbourNode.Position == InvalidPosition)
		{
			// Neighbor nodes dont have position
			const uint32 LaneIndex = NeighbourNode.NodeRef;
			const FZoneLaneData& LaneData = ZoneStorage.Lanes[LaneIndex];
			const FVector LaneEndPoint = ZoneStorage.LanePoints[LaneData.PointsEnd - 1];

			ensure(EndNode.Position != InvalidPosition);
			return FVector::Distance(LaneEndPoint, EndNode.Position);
		}
		else
		{
			ensure(NeighbourNode.Position != InvalidPosition && EndNode.Position != InvalidPosition);
			return FVector::Distance(NeighbourNode.Position, EndNode.Position);
		}
	}

	FVector::FReal GetTraversalCost(const FZoneGraphAStarNode& CurNode, const FZoneGraphAStarNode& NeighbourNode) const
	{
		const FZoneLaneData& CurLane = ZoneStorage.Lanes[CurNode.NodeRef];
		const FZoneLaneData& NeighbourLane = ZoneStorage.Lanes[NeighbourNode.NodeRef];
		const bool bDifferentZones = (CurLane.ZoneIndex != NeighbourLane.ZoneIndex);

		if (bDifferentZones)
		{
			// Nodes are in different zones

			float TravelLength = 0.f;
			if (IsStart(CurNode))
			{
				float CurLaneLength = -1.f;
				UE::ZoneGraph::Query::GetLaneLength(ZoneStorage, StartLocation.LaneHandle.Index, CurLaneLength);
				TravelLength += (CurLaneLength - StartLocation.DistanceAlongLane);
			}

			if (IsEnd(NeighbourNode))
			{
				TravelLength += EndLocation.DistanceAlongLane;
			}
			else
			{
				// Most common case, take the lane length of NeighbourNode.
				// The traversal cost is the cost of reaching the end of the neighbour node.
				float LaneLength = -1.f;
				UE::ZoneGraph::Query::GetLaneLength(ZoneStorage, NeighbourNode.NodeRef, LaneLength);
				TravelLength += LaneLength;
			}
			return TravelLength;
		}
		else
		{
			// Special case, nodes are in the same zone

			if (IsStart(CurNode))
			{
				// Rare case, same zone, get nearest point on NeighbourNode 
				const float SearchDistance = 5.f * CurLane.Width; // Arbitrary search dist
				FZoneGraphLaneLocation LocationOnNeighbourNode;
				float DistanceSqr = 0.f;
				FBox Bounds(CurNode.Position, CurNode.Position);
				Bounds = Bounds.ExpandBy(SearchDistance);
				UE::ZoneGraph::Query::FindNearestLocationOnLane(
					ZoneStorage,
					FZoneGraphLaneHandle(NeighbourNode.NodeRef, ZoneStorage.DataHandle),
					Bounds,
					LocationOnNeighbourNode,
					DistanceSqr
				);

				if (IsEnd(NeighbourNode))
				{
					return FMath::Abs(EndLocation.DistanceAlongLane - LocationOnNeighbourNode.DistanceAlongLane);
				}
				else
				{
					// Get distance from closest point on NeighbourNode to the end
					float Length = -1.f;
					UE::ZoneGraph::Query::GetLaneLength(ZoneStorage, NeighbourNode.NodeRef, Length);
					return Length - LocationOnNeighbourNode.DistanceAlongLane;
				}
			}
			else
			{
				UE_LOGF(LogAStar, Warning, "Not handled");
				return TNumericLimits<FVector::FReal>::Max();
			}
		}
	}

	inline bool IsTraversalAllowed(const FNodeRef StartNodeRef, const FNodeRef& Neighbour) const
	{
		const FZoneGraphTagMask& LaneTagMask = ZoneStorage.Lanes[Neighbour].Tags;
		return ZoneTagFilter.Pass(LaneTagMask);
	}

	inline bool WantsPartialSolution() const { return false; }
	inline bool ShouldIncludeStartNodeInPath() const { return true; }

protected:
	const FZoneGraphStorage& ZoneStorage;
	const FZoneGraphTagFilter ZoneTagFilter;
	const FZoneGraphLaneLocation StartLocation;
	const FZoneGraphLaneLocation EndLocation;
};

/** A Star algorithm using lanes on zone graph */
struct FZoneGraphAStar : public FGraphAStar<FZoneGraphAStarWrapper, FGraphAStarDefaultPolicy, FZoneGraphAStarNode>
{
	typedef FGraphAStar<FZoneGraphAStarWrapper, FGraphAStarDefaultPolicy, FZoneGraphAStarNode> Super;

	FZoneGraphAStar(const FZoneGraphAStarWrapper& Graph)
		: Super(Graph)
	{}
};

#undef UE_API
