// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/Box.h"
#include "Containers/StridedView.h"
#include "ZoneGraphBVTree.generated.h"

/** Quantized BV-tree node. */
USTRUCT()
struct FZoneGraphBVNode
{
	GENERATED_BODY()

	FZoneGraphBVNode() = default;
	FZoneGraphBVNode(const uint16 InMinX, const uint16 InMinY, const uint16 InMinZ, const uint16 InMaxX, const uint16 InMaxY, const uint16 InMaxZ)
		: MinX(InMinX)
		, MinY(InMinY)
		, MinZ(InMinZ)
		, MaxX(InMaxX)
		, MaxY(InMaxY)
		, MaxZ(InMaxZ)
	{
	}
	
	bool DoesOverlap(const FZoneGraphBVNode& Other) const
	{
		if (MinX > Other.MaxX || MinY > Other.MaxY || MinZ > Other.MaxZ) return false;
		if (MaxX < Other.MinX || MaxY < Other.MinY || MaxZ < Other.MinZ) return false;
		return true;
	}

	/** Quantized node bounds */
	UPROPERTY()
	uint16 MinX = 0;

	UPROPERTY()
	uint16 MinY = 0;

	UPROPERTY()
	uint16 MinZ = 0;

	UPROPERTY()
	uint16 MaxX = 0;

	UPROPERTY()
	uint16 MaxY = 0;

	UPROPERTY()
	uint16 MaxZ = 0;

	/** Item Index, or if negative, the node is internal and the index is relative index to next sibling. */
	UPROPERTY()
	int32 Index = 0;
};

/** Quantized BV-Tree */
USTRUCT()
struct FZoneGraphBVTree
{
	GENERATED_BODY()

	/** Build BV-tree from boxes, the index of the box in the array will be the index of the query result. */
	void Build(TStridedView<const FBox> Boxes);

	/** Queries the BV-tree, calls Function on each child node which bounds overlap the query bounds. */
	template<typename TFunc>
	void Query(const FBox& Bounds, TFunc&& Function) const
	{
		const FZoneGraphBVNode QueryBounds = CalcNodeBounds(Bounds);
		const int32 LimitIndex = Nodes.Num();
		int32 NodeIndex = 0;
		
		while (NodeIndex < LimitIndex)
		{
			const FZoneGraphBVNode& Node = Nodes[NodeIndex];
			const bool bOverlap = Node.DoesOverlap(QueryBounds);
			const bool bLeafNode = (Node.Index >= 0);

			if (bLeafNode && bOverlap)
			{
				Function(Node);
			}

			NodeIndex += (bOverlap || bLeafNode) ? 1 : -Node.Index;
		}
	}

	/** Queries the BV-tree, return index of overlapping items. */
	void Query(const FBox& Bounds, TArray<int32>& OutItems) const;

	/** @return number of nodes in the tree */
	int32 GetNumNodes() const { return Nodes.Num(); }

	/** @return quantization origin */
	const FVector& GetOrigin() const { return Origin; }
	
	/** @return world-to-quantized scaling factor */ 
	float GetQuantizationScale() const { return QuantizationScale; }
	
	/** @return Tree nodes. */
	TConstArrayView<FZoneGraphBVNode> GetNodes() const { return Nodes; }

	/** @return quantized node bounds based on world bounds. */
	FZoneGraphBVNode CalcNodeBounds(const FBox& Box) const
	{
		const FVector QuantizedBoxSize(MaxQuantizedCoord);
		const FVector LocalMin = ClampVector((Box.Min - Origin) * QuantizationScale, FVector::ZeroVector, QuantizedBoxSize);
		const FVector LocalMax = ClampVector((Box.Max - Origin) * QuantizationScale, FVector::ZeroVector, QuantizedBoxSize);
		return FZoneGraphBVNode(uint16(LocalMin.X), uint16(LocalMin.Y), uint16(LocalMin.Z),
								uint16(LocalMax.X + 1), uint16(LocalMax.Y + 1), uint16(LocalMax.Z + 1));
	}

	/** @return world bounding box of a node. */
	FBox CalcWorldBounds(const FZoneGraphBVNode& Node) const
	{
		const float UnquantizedScale = QuantizationScale > KINDA_SMALL_NUMBER ? 1.0f / QuantizationScale : 0.0f; // Scale from quantized to unquantized coordinates.
		return FBox(Origin + FVector(Node.MinX * UnquantizedScale, Node.MinY * UnquantizedScale, Node.MinZ* UnquantizedScale), 
					Origin + FVector(Node.MaxX * UnquantizedScale, Node.MaxY * UnquantizedScale, Node.MaxZ* UnquantizedScale));
	}

protected:

	/** Max quantized coordinate value during conversion, the scale is truncated to account for the expansion in CalcNodeBounds(). */
	static constexpr float MaxQuantizedCoord = float(MAX_uint16 - 1);

	/** Quantization range origin */
	UPROPERTY()
	FVector Origin = FVector::ZeroVector;

	/** Scale to convert from world coordinates to quantized range. */
	UPROPERTY()
	float QuantizationScale = 0.0f;

	/** BV-tree nodes. */
	UPROPERTY()
	TArray<FZoneGraphBVNode> Nodes;
};
