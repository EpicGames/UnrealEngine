// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ZoneGraphTypes.h"
#include "HierarchicalHashGrid2D.h"
#include "ZoneGraphBuilder.generated.h"

#define UE_API ZONEGRAPH_API

class AZoneGraphData;
class UZoneShapeComponent;
struct FZoneShapeLaneInternalLink;

typedef THierarchicalHashGrid2D<5, 4> FZoneGraphBuilderHashGrid2D;	// 5 levels of hierarchy, 4 ration between levels (biggest bucket 4^5 = 1024 cells)

USTRUCT()
struct FZoneGraphBuilderRegisteredComponent
{
	GENERATED_BODY()

	FZoneGraphBuilderRegisteredComponent() = default;
	FZoneGraphBuilderRegisteredComponent(UZoneShapeComponent* InComponent) : Component(InComponent) {}

	UPROPERTY()
	TObjectPtr<UZoneShapeComponent> Component = nullptr;

	UPROPERTY()
	uint32 ShapeHash = 0;

	FZoneGraphBuilderHashGrid2D::FCellLocation CellLoc;
};

// Build time data for UZoneShapeComponent
USTRUCT()
struct FZoneShapeComponentBuildData
{
	GENERATED_BODY()

	UPROPERTY()
	int32 ZoneIndex = 0;

	UPROPERTY()
	TArray<FZoneGraphLaneHandle> Lanes;
};

// Build time data, can be used to map things between editor representation and baked data.
USTRUCT()
struct FZoneGraphBuildData
{
	GENERATED_BODY()

	void Reset()
	{
		ZoneShapeComponentBuildData.Reset();
	}

	UPROPERTY()
	TMap<TObjectPtr<const UZoneShapeComponent>, FZoneShapeComponentBuildData> ZoneShapeComponentBuildData;
};

USTRUCT()
struct FZoneGraphBuilder
{
	GENERATED_BODY()

public:
	UE_API FZoneGraphBuilder();
	UE_API ~FZoneGraphBuilder();

	UE_API void RegisterZoneShapeComponent(UZoneShapeComponent& ShapeComp);
	UE_API void UnregisterZoneShapeComponent(UZoneShapeComponent& ShapeComp);
	UE_API void OnZoneShapeComponentChanged(UZoneShapeComponent& ShapeComp);
	const TArray<FZoneGraphBuilderRegisteredComponent>& GetRegisteredZoneShapeComponents() const { return ShapeComponents; }

	bool NeedsRebuild() const { return bIsDirty; }

	/** Builds zone graph for each zone graph data based on registered shapes.
	 * @param AllZoneGraphData All Zone Graph data to update.
	 * @param bForceRebuild If set will build graph even if it inputs have not changed.
	 */
	UE_API void BuildAll(const TArray<AZoneGraphData*>& AllZoneGraphData, const bool bForceRebuild);

	UE_API void FindShapeConnections(const UZoneShapeComponent& SourceShapeComp, TArray<FZoneShapeConnection>& OutShapeConnections) const;

	/** Converts single zone shape into a zone storage, used in UI for editing and rendering.
	*/
	static UE_API void BuildSingleShape(const UZoneShapeComponent& ShapeComp, const FMatrix& LocalToWorld, FZoneGraphStorage& OutZoneStorage);

	/** Returns items that potentially touch the bounds in the HashGrid. Operates on grid level, can have false positives.
	 * @param Bounds - Query bounding box.
	 * @param OutResults - Result of the query, IDs of potentially overlapping items.
	 */
	UE_API void QueryHashGrid(const FBox& Bounds, TArray<FZoneGraphBuilderHashGrid2D::ItemIDType>& OutResults);

protected:
	UE_API void Build(AZoneGraphData& ZoneGraphData);
	UE_API void RequestRebuild();
	UE_API void OnLaneProfileChanged(const FZoneLaneProfileRef& ChangedLaneProfileRef);
	UE_API uint32 CalculateCombinedShapeHash(const AZoneGraphData& ZoneGraphData) const;
	static UE_API void AppendShapeToZoneStorage(const UZoneShapeComponent& ShapeComp, const FMatrix& LocalToWorld, FZoneGraphStorage& OutZoneStorage, TArray<FZoneShapeLaneInternalLink>& OutInternalLinks, FZoneGraphBuildData* InBuildData = nullptr);
	static UE_API void ConnectLanes(TArray<FZoneShapeLaneInternalLink>& InternalLinks, FZoneGraphStorage& OutZoneStorage);

	UPROPERTY(Transient)
	TArray<FZoneGraphBuilderRegisteredComponent> ShapeComponents;

	UPROPERTY(Transient)
	TArray<int32> ShapeComponentsFreeList;

	UPROPERTY(Transient)
	TMap<TObjectPtr<UZoneShapeComponent>, int32> ShapeComponentToIndex;

	UPROPERTY(Transient)
	FZoneGraphBuildData BuildData;

#if WITH_EDITOR
	FDelegateHandle OnBuildSettingsChangedHandle;
	FDelegateHandle OnTagsChangedHandle;
	FDelegateHandle OnLaneProfileChangedHandle;
#endif

	FZoneGraphBuilderHashGrid2D HashGrid;

	bool bSkipHashCheck = false;
	bool bIsDirty = false;
};

#undef UE_API
