// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "ZoneGraphTypes.h"
#include "ZoneGraphData.generated.h"

#define UE_API ZONEGRAPH_API

class UZoneGraphRenderingComponent;

UCLASS(MinimalAPI, config = ZoneGraph, defaultconfig, NotBlueprintable)
class AZoneGraphData : public AActor
{
	GENERATED_BODY()
public:
	UE_API AZoneGraphData(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	//~ Begin UObject/AActor Interface
	UE_API virtual void PostActorCreated() override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void Destroyed() override;
	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	UE_API virtual void PreRegisterAllComponents() override;
	UE_API virtual void PostUnregisterAllComponents() override;

#if WITH_EDITOR
	UE_API virtual void PostEditUndo() override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual bool CanChangeIsSpatiallyLoadedFlag() const override { return false; }
#endif // WITH_EDITOR
	//~ End UObject/AActor Interface

	inline bool IsDrawingEnabled() const { return bEnableDrawing; }
	UE_API void UpdateDrawing() const;

	inline bool IsRegistered() const { return bRegistered; }
	UE_API void OnRegistered(const FZoneGraphDataHandle DataHandle);
	UE_API void OnUnregistered();

	// TODO: I wonder if the storage is unnecessary indirection?
	FZoneGraphStorage& GetStorageMutable() { return ZoneStorage; }
	const FZoneGraphStorage& GetStorage() const { return ZoneStorage; }
	FCriticalSection& GetStorageLock() const { return ZoneStorageLock; }

	UE_API FBox GetBounds() const;

	/** @return Combined hash of all ZoneShapes that were used to build the data. */
	uint32 GetCombinedShapeHash() const { return CombinedShapeHash; }

	/** Sets Combined hash of all ZoneShapes that were used to build the data. */
	void SetCombinedShapeHash(const uint32 Hash) { CombinedShapeHash = Hash; }

protected:
	UE_API bool RegisterWithSubsystem();
	UE_API bool UnregisterWithSubsystem();

	bool bRegistered;

	/** if set to true then this zone graph data will be drawing itself when requested as part of "show navigation" */
	UPROPERTY(Transient, EditAnywhere, Category = Display)
	bool bEnableDrawing;

	UPROPERTY(transient, duplicatetransient)
	TObjectPtr<UZoneGraphRenderingComponent> RenderingComp;

	UPROPERTY()
	FZoneGraphStorage ZoneStorage;

	/** Critical section to prevent rendering of the zone graph storage data while it's getting rebuilt */
	mutable FCriticalSection ZoneStorageLock;

	/** Combined hash of all ZoneShapes that were used to build the data. */
	UPROPERTY()
	uint32 CombinedShapeHash = 0;
};

#undef UE_API
