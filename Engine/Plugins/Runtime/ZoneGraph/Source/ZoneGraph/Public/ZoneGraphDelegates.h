// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class AZoneGraphData;
struct FZoneGraphBuildData;
struct FZoneLaneProfileRef;

namespace UE::ZoneGraphDelegates
{

#if WITH_EDITOR
/** Called when build is completed. */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnZoneGraphDataBuildDone, const FZoneGraphBuildData& /*BuildData*/);
extern ZONEGRAPH_API FOnZoneGraphDataBuildDone OnZoneGraphDataBuildDone;

/** Called when tags have changed. */
DECLARE_MULTICAST_DELEGATE(FOnZoneGraphTagsChanged);
extern ZONEGRAPH_API FOnZoneGraphTagsChanged OnZoneGraphTagsChanged;

/** Called when name in a lane profile has changed. */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnZoneGraphLaneProfileChanged, const FZoneLaneProfileRef& /*ChangedLaneProfileRef*/);
extern ZONEGRAPH_API FOnZoneGraphLaneProfileChanged OnZoneGraphLaneProfileChanged;

/** Called when build settings change. */
DECLARE_MULTICAST_DELEGATE(FOnZoneGraphBuildSettingsChanged);
extern ZONEGRAPH_API FOnZoneGraphBuildSettingsChanged OnZoneGraphBuildSettingsChanged;

/** Called when rebuild is requested */
DECLARE_MULTICAST_DELEGATE(FOnZoneGraphRequestRebuild);
extern ZONEGRAPH_API FOnZoneGraphRequestRebuild OnZoneGraphRequestRebuild;
#endif // WITH_EDITOR

DECLARE_MULTICAST_DELEGATE_OneParam(FOnZoneGraphData, const AZoneGraphData* /*ZoneGraphData*/);

/**
 * Called when a zone graph gets registered to the zone graph subsystem.
 * It is the listener responsibility to validate that the graph belongs to its associated world if needed.
 */
extern ZONEGRAPH_API FOnZoneGraphData OnPostZoneGraphDataAdded;

 /**
 * Called when a zone graph gets unregistered from the zone graph subsystem.
 * It is the listener responsibility to validate that the graph belongs to its associated world if needed.
 */
extern ZONEGRAPH_API FOnZoneGraphData OnPreZoneGraphDataRemoved;

} // UE::ZoneGraphDelegates
