// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/ArchiveObjectCrc32.h"

#if WITH_EDITORONLY_DATA

#include "UObject/UnrealType.h"

/**
 * Archive based object hashing to be used with ZoneGraph data calculation.
 * If a property is has "IncludeInHash" meta tag, any of it's child properties will be included in the hash.
 * Editor only s it relies on meta data.
 */
class FZoneGraphObjectCRC32 : public FArchiveObjectCrc32
{
public:

	/** @return True, if any of the properties in the chain has the Key set */
	bool HasMetaDataInChain(const FProperty* InProperty, FName Key) const
	{
		bool bHasMetaData = InProperty->HasMetaData(Key);
		if (!bHasMetaData)
		{
			TArray<FProperty*> PropertyChain;
			GetSerializedPropertyChain(PropertyChain);
			for (FProperty* Prop : PropertyChain)
			{
				if (Prop->HasMetaData(Key))
				{
					bHasMetaData = true;
					break;
				}
			}
		}
		return bHasMetaData;
	}

	virtual bool ShouldSkipProperty(const FProperty* InProperty) const override
	{
		static const FName IncludeInHashName(TEXT("IncludeInHash"));
		check(InProperty);
		return FArchiveObjectCrc32::ShouldSkipProperty(InProperty) || InProperty->HasAllPropertyFlags(CPF_Transient) || !HasMetaDataInChain(InProperty, IncludeInHashName);
	}
};

#endif // WITH_EDITORONLY_DATA