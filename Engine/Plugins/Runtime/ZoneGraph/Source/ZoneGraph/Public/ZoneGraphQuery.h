// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ZoneGraphTypes.h"

namespace UE::ZoneGraph::Query
{

/**  Returns the length of a specific lane. */
ZONEGRAPH_API bool GetLaneLength(const FZoneGraphStorage& Storage, const uint32 LaneIndex, float& OutLength);

/**  Returns the length of a specific lane. */
ZONEGRAPH_API bool GetLaneLength(const FZoneGraphStorage& Storage, const FZoneGraphLaneHandle LaneHandle, float& OutLength);

/**  Returns the width of a specific lane. */
ZONEGRAPH_API bool GetLaneWidth(const FZoneGraphStorage& Storage, const uint32 LaneIndex, float& OutWidth);

/**  Returns the width of a specific lane. */
ZONEGRAPH_API bool GetLaneWidth(const FZoneGraphStorage& Storage, const FZoneGraphLaneHandle LaneHandle, float& OutWidth);

/**  Returns the tags of a specific lane. */
ZONEGRAPH_API bool GetLaneTags(const FZoneGraphStorage& Storage, const FZoneGraphLaneHandle& LaneHandle, FZoneGraphTagMask& OutTags);

/**  Returns the tags of a specific lane. */
ZONEGRAPH_API bool GetLaneTags(const FZoneGraphStorage& Storage, const uint32 LaneIndex, FZoneGraphTagMask& OutTags);

/**  Returns links to connected lanes of a specific lane. Flags are only used for adjacent lanes. */
ZONEGRAPH_API bool GetLinkedLanes(const FZoneGraphStorage& Storage, const uint32 LaneIndex, const EZoneLaneLinkType Types, const EZoneLaneLinkFlags IncludeFlags, const EZoneLaneLinkFlags ExcludeFlags, TArray<FZoneGraphLinkedLane>& OutLinkedLanes);

/**  Returns links to connected lanes of a specific lane. Flags are only used for adjacent lanes. */
ZONEGRAPH_API bool GetLinkedLanes(const FZoneGraphStorage& Storage, const FZoneGraphLaneHandle LaneHandle, const EZoneLaneLinkType Types, const EZoneLaneLinkFlags IncludeFlags, const EZoneLaneLinkFlags ExcludeFlags, TArray<FZoneGraphLinkedLane>& OutLinkedLanes);

/**  Returns first link matching the connection and flags. Flags are only used for adjacent lanes.. */
ZONEGRAPH_API bool GetFirstLinkedLane(const FZoneGraphStorage& Storage, const uint32 LaneIndex, const EZoneLaneLinkType Types, const EZoneLaneLinkFlags IncludeFlags, const EZoneLaneLinkFlags ExcludeFlags, FZoneGraphLinkedLane& OutLinkedLane);

/**  Returns first link matching the connection and flags. Flags are only used for adjacent lanes. */
ZONEGRAPH_API bool GetFirstLinkedLane(const FZoneGraphStorage& Storage, const FZoneGraphLaneHandle LaneHandle, const EZoneLaneLinkType Types, const EZoneLaneLinkFlags IncludeFlags, const EZoneLaneLinkFlags ExcludeFlags, FZoneGraphLinkedLane& OutLinkedLane);

/**  Moves LaneLocation along a lane. */
ZONEGRAPH_API bool AdvanceLaneLocation(const FZoneGraphStorage& Storage, const FZoneGraphLaneLocation& InLaneLocation, const float AdvanceDistance, FZoneGraphLaneLocation& OutLaneLocation);

/**  Moves LaneLocation along a lane and provides list of locations containing all segments start location along the way and final interpolated location. */
ZONEGRAPH_API bool AdvanceLaneLocation(const FZoneGraphStorage& Storage, const FZoneGraphLaneLocation& InLaneLocation, const float AdvanceDistance, TArray<FZoneGraphLaneLocation>& OutLaneLocations);
	
/**  Returns location at a distance along a specific lane. */
ZONEGRAPH_API bool CalculateLocationAlongLane(const FZoneGraphStorage& Storage, const uint32 LaneIndex, const float Distance, FZoneGraphLaneLocation& OutLaneLocation);

/**  Returns location at a distance along a specific lane. */
ZONEGRAPH_API bool CalculateLocationAlongLane(const FZoneGraphStorage& Storage, const FZoneGraphLaneHandle LaneHandle, const float Distance, FZoneGraphLaneLocation& OutLaneLocation);

/**  Returns location at a given lane lenght ratio [0..1] along a specific lane. */
ZONEGRAPH_API bool CalculateLocationAlongLaneFromRatio(const FZoneGraphStorage& Storage, const uint32 LaneIndex, const float Ratio, FZoneGraphLaneLocation& OutLaneLocation);

/**  Returns location at a given lane lenght ratio [0..1] along a specific lane. */
ZONEGRAPH_API bool CalculateLocationAlongLaneFromRatio(const FZoneGraphStorage& Storage, const FZoneGraphLaneHandle LaneHandle, const float Ratio, FZoneGraphLaneLocation& OutLaneLocation);

/**  Returns segment index at distance along a specific lane. */
ZONEGRAPH_API bool CalculateLaneSegmentIndexAtDistance(const FZoneGraphStorage& Storage, const uint32 LaneIndex, const float Distance, int32& OutSegmentIndex);

/**  Returns segment index at distance along a specific lane. */
ZONEGRAPH_API bool CalculateLaneSegmentIndexAtDistance(const FZoneGraphStorage& Storage, const FZoneGraphLaneHandle LaneHandle, const float Distance, int32& OutSegmentIndex);

/**  Find nearest location on a specific lane. */
ZONEGRAPH_API bool FindNearestLocationOnLane(const FZoneGraphStorage& Storage, const FZoneGraphLaneHandle LaneHandle, const FBox& Bounds, FZoneGraphLaneLocation& OutLaneLocation, float& OutDistanceSqr);

/**  Find nearest location on a specific lane. */
ZONEGRAPH_API bool FindNearestLocationOnLane(const FZoneGraphStorage& Storage, const FZoneGraphLaneHandle LaneHandle, const FVector& Center, const float Range, FZoneGraphLaneLocation& OutLaneLocation, float& OutDistanceSqr);

/**  Finds nearest lane in ZoneGraph Storage.  */
ZONEGRAPH_API bool FindNearestLane(const FZoneGraphStorage& Storage, const FBox& Bounds, const FZoneGraphTagFilter TagFilter, FZoneGraphLaneLocation& OutLaneLocation, float& OutDistanceSqr);

/**  Finds overlapping lanes in ZoneGraph Storage.  */
ZONEGRAPH_API bool FindOverlappingLanes(const FZoneGraphStorage& Storage, const FBox& Bounds, const FZoneGraphTagFilter TagFilter, TArray<FZoneGraphLaneHandle>& OutLanes);
	
/**  Find sections of lanes fully overlapping (including lane width) in ZoneGraph Storage. */
ZONEGRAPH_API bool FindLaneOverlaps(const FZoneGraphStorage& Storage, const FVector& Center, const float Radius, const FZoneGraphTagFilter TagFilter, TArray<FZoneGraphLaneSection>& OutLaneSections);

} // UE::ZoneGraph::Query