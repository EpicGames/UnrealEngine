// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ZoneGraphTypes.h"
#include "Components/PrimitiveComponent.h"
#include "DebugRenderSceneProxy.h"
#include "ZoneGraphRenderingComponent.generated.h"

#define UE_API ZONEGRAPH_API

class UZoneGraphRenderingComponent;
class AZoneGraphData;

// exported to API for GameplayDebugger module
class FZoneGraphSceneProxy : public FDebugRenderSceneProxy
{
public:
	UE_API virtual SIZE_T GetTypeHash() const override;

	UE_API FZoneGraphSceneProxy(const UPrimitiveComponent& InComponent, const AZoneGraphData& ZoneGraph);
	UE_API virtual ~FZoneGraphSceneProxy();

	UE_API virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;

	struct FZoneVisibility
	{
		bool bVisible = false;
		bool bDetailsVisible = false;
		float Alpha = 1.0f;
	};
	struct FDrawDistances
	{
		float MinDrawDistanceSqr = 0.f;
		float MaxDrawDistanceSqr = FLT_MAX;
		float FadeDrawDistanceSqr = FLT_MAX;
		float DetailDrawDistanceSqr = FLT_MAX;
	};
	static UE_API FDrawDistances GetDrawDistances(const float MinDrawDistance, const float MaxDrawDistance);
	static UE_API FZoneVisibility CalculateZoneVisibility(const FDrawDistances& Distances, const FVector Origin, const FVector Position);
	static UE_API bool ShouldRenderZoneGraph(const FSceneView& View);

protected:

	UE_API virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;

	virtual uint32 GetMemoryFootprint(void) const override { return sizeof(*this) + GetAllocatedSize(); }

	uint32 GetAllocatedSize(void) const
	{
		return FDebugRenderSceneProxy::GetAllocatedSize();
	}

private:

	TWeakObjectPtr<UZoneGraphRenderingComponent> WeakRenderingComponent;
	bool bSkipDistanceCheck;
};

UCLASS(MinimalAPI, hidecategories = Object, editinlinenew)
class UZoneGraphRenderingComponent : public UPrimitiveComponent
{
	GENERATED_BODY()

public:
	UE_API UZoneGraphRenderingComponent(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	//~ Begin UPrimitiveComponent Interface
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	UE_API virtual void OnRegister()  override;
	UE_API virtual void OnUnregister()  override;
	//~ End UPrimitiveComponent Interface

	//~ Begin USceneComponent Interface
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ End USceneComponent Interface

	void ForceUpdate() { bForceUpdate = true; }
	bool IsForcingUpdate() const { return bForceUpdate; }

	static UE_API bool IsNavigationShowFlagSet(const UWorld* World);

protected:

	UE_API void CheckDrawFlagTimerFunction();

protected:
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	FDelegateHandle DebugTextDrawingDelegateHandle;
#endif

	bool bPreviousShowNavigation;
	bool bForceUpdate;
	FTimerHandle TimerHandle;
};

#undef UE_API
