// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SceneManagement.h"
#include "ZoneGraphTypes.h"

class FDebugRenderSceneProxy;

namespace UE::ZoneGraph::RenderingUtilities
{

struct FLaneHighlight
{
	bool IsValid() const { return Width > 0.0f; }

	FVector Position = FVector::ZeroVector;
	FQuat Rotation = FQuat::Identity;
	float Width = 0.0f;
};

struct FZoneDrawAnnotator
{
	TOptional<FZoneGraphTagMask> VisualizedTagsOverride;
	TOptional<FZoneGraphTagMask> ZoneTagsOverride;
	TOptional<TFunctionRef<FZoneGraphTagMask(int32 LaneIndex, const FZoneLaneData& Lane)>> GetLaneTagsOverride;
};

ZONEGRAPH_API FColor GetTagMaskColor(const FZoneGraphTagMask TagMask, TConstArrayView<FZoneGraphTagInfo> TagInfo);
ZONEGRAPH_API void DrawZoneBoundary(const FZoneGraphStorage& ZoneStorage, int32 ZoneIndex, FPrimitiveDrawInterface* PDI, const FMatrix& LocalToWorld, const float LineThickness, const float DepthBias, const float Alpha, const FZoneDrawAnnotator& Annotator = { });
ZONEGRAPH_API void DrawLane(const FZoneGraphStorage& ZoneStorage, FPrimitiveDrawInterface* PDI, const FZoneGraphLaneHandle& LaneHandle, const FColor Color, const float LineThickness = 5.f, const FVector& Offset = FVector::ZeroVector);
ZONEGRAPH_API void DrawLane(const FZoneGraphStorage& ZoneStorage, FPrimitiveDrawInterface* PDI, const FZoneGraphLaneLocation& InStartLocation, const FZoneGraphLaneLocation& InEndLocation, const FColor Color, const float LineThickness = 5.f, const FVector& Offset = FVector::ZeroVector);
ZONEGRAPH_API void DrawLanes(const FZoneGraphStorage& ZoneStorage, FPrimitiveDrawInterface* PDI, const TArray<FZoneGraphLaneHandle>& Lanes, const FColor Color, const float LineThickness = 5.f, const FVector& Offset = FVector::ZeroVector);
ZONEGRAPH_API void DrawLinkedLanes(const FZoneGraphStorage& ZoneStorage, FPrimitiveDrawInterface* PDI, const FZoneGraphLaneHandle& SourceLaneHandle, const TArray<FZoneGraphLinkedLane>& LinkedLanes, const float LineThickness = 5.f);
ZONEGRAPH_API void DrawLanePath(const FZoneGraphStorage& ZoneStorage, FPrimitiveDrawInterface* PDI, const FZoneGraphLanePath& LanePath, const FColor Color, const float LineThickness = 5.f);
ZONEGRAPH_API void DrawZoneLanes(const FZoneGraphStorage& ZoneStorage, int32 ZoneIndex, FPrimitiveDrawInterface* PDI, const FMatrix& LocalToWorld, const float LineThickness, const float DepthBias, const float Alpha, const bool bDrawDetails = true, const FLaneHighlight& LaneHighlight = FLaneHighlight(), const FZoneDrawAnnotator& Annotator = { });

ZONEGRAPH_API void DrawZoneShapeConnector(const FZoneShapeConnector& Connector, const FZoneShapeConnection* Connection, FPrimitiveDrawInterface* PDI, const FMatrix& LocalToWorld, const float DepthBias);
ZONEGRAPH_API void DrawZoneShapeConnectors(TConstArrayView<FZoneShapeConnector> Connectors, TConstArrayView<FZoneShapeConnection> Connections, FPrimitiveDrawInterface* PDI, const FMatrix& LocalToWorld, const float DepthBias);

ZONEGRAPH_API void DrawLaneDirections(const FZoneGraphStorage& ZoneStorage, FPrimitiveDrawInterface* PDI, const FZoneGraphLaneHandle LaneHandle, const FColor Color);
ZONEGRAPH_API void DrawLaneSmoothing(const FZoneGraphStorage& ZoneStorage, FPrimitiveDrawInterface* PDI, const FZoneGraphLaneHandle LaneHandle, const FColor Color);

ZONEGRAPH_API void AppendLane(FDebugRenderSceneProxy* DebugProxy, const FZoneGraphStorage& ZoneStorage, const FZoneGraphLaneHandle& LaneHandle, const FColor Color, const float LineThickness = 5.f, const FVector& Offset = FVector::ZeroVector);
ZONEGRAPH_API void AppendLane(FDebugRenderSceneProxy* DebugProxy, const FZoneGraphStorage& ZoneStorage, const FZoneGraphLaneLocation& InStartLocation, const FZoneGraphLaneLocation& InEndLocation, const FColor Color, const float LineThickness = 5.f, const FVector& Offset = FVector::ZeroVector);
ZONEGRAPH_API void AppendLaneSection(FDebugRenderSceneProxy* DebugProxy, const FZoneGraphStorage& ZoneStorage, const FZoneGraphLaneSection& Section, const FColor Color, const float LineThickness = 5.f, const FVector& Offset = FVector::ZeroVector);	

} // namespace UE::ZoneGraph::RenderingUtilities
