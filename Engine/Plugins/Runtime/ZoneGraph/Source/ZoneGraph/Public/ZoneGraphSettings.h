// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Engine/DeveloperSettings.h"
#include "ZoneGraphTypes.h"
#include "ZoneGraphSettings.generated.h"

#define UE_API ZONEGRAPH_API

namespace UE::ZoneGraph::Helpers
{
ZONEGRAPH_API FName GetTagName(const FZoneGraphTag Tag);
ZONEGRAPH_API FString GetTagMaskString(const FZoneGraphTagMask TagMask, const TCHAR* Separator);
}

/**
 * Implements the settings for the ZoneGraph plugin.
 */
UCLASS(MinimalAPI, config = Plugins, defaultconfig, DisplayName = "Zone Graph")
class UZoneGraphSettings : public UDeveloperSettings
{
	GENERATED_BODY()
public:
	UE_API UZoneGraphSettings(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	const TArray<FZoneLaneProfile>& GetLaneProfiles() const { return LaneProfiles; }

	UE_API const FZoneLaneProfile* GetLaneProfileByRef(const FZoneLaneProfileRef& LaneProfileRef) const;
	UE_API const FZoneLaneProfile* GetLaneProfileByID(const FGuid& ID) const;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;

	UE_API const FZoneLaneProfile* GetDefaultLaneProfile() const;
#endif

	UE_API TConstArrayView<FZoneGraphTagInfo> GetTagInfos() const;
	UE_API void GetValidTagInfos(TArray<FZoneGraphTagInfo>& OutInfos) const;

	FZoneGraphTagMask GetVisualizedTags() const { return VisualizedTags; }

	const FZoneGraphBuildSettings& GetBuildSettings() const { return BuildSettings; }

	float GetShapeMaxDrawDistance() const { return ShapeMaxDrawDistance; }

	bool ShouldBuildZoneGraphWhileEditing() const { return bBuildZoneGraphWhileEditing; }
	
#if WITH_EDITOR
	/** Calculates hash values from all build settings Can be used to determine if the settings have changed between builds.
	 * Use property meta tag "ExcludeFromHash" to exclude non build related properties.
	 * @return Hash value of all build related settings. */
	UE_API uint32 GetBuildHash() const;
#endif

protected:
	UPROPERTY(EditAnywhere, config, Category = ZoneGraph, meta = (IncludeInHash));
	TArray<FZoneLaneProfile> LaneProfiles;

	UPROPERTY(EditAnywhere, config, Category = ZoneGraph)
	FZoneGraphTagInfo Tags[static_cast<int32>(EZoneGraphTags::MaxTags)];

	/** Tags which affect visualization (i.e. color of lanes). */
	UPROPERTY(EditAnywhere, config, Category = ZoneGraph)
	FZoneGraphTagMask VisualizedTags = FZoneGraphTagMask::All;

	/** Max draw distance for shapes visualization. */
	UPROPERTY(EditAnywhere, config, Category = ZoneGraph)
	float ShapeMaxDrawDistance = 20000.0f;

	UPROPERTY(EditAnywhere, config, Category = ZoneGraph, meta = (IncludeInHash))
	FZoneGraphBuildSettings BuildSettings;

	/** When set to true ZoneGraph will build as it is being edited. */
	UPROPERTY(EditAnywhere, config, Category = ZoneGraph)
	bool bBuildZoneGraphWhileEditing = false;
};

#undef UE_API
