// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ZoneGraphTypes.h"
#include "ZoneGraphData.h"
#include "Subsystems/WorldSubsystem.h"
#include "Mass/ExternalSubsystemTraits.h"
#if WITH_EDITOR
#include "ZoneGraphBuilder.h"
#endif
#include "ZoneGraphSubsystem.generated.h"

class UWorld;
class UZoneShapeComponent;


// Struct representing registered ZoneGraph data in the subsystem.
USTRUCT()
struct FRegisteredZoneGraphData
{
	GENERATED_BODY()

	void Reset(int32 InGeneration = 1)
	{
		ZoneGraphData = nullptr;
		bInUse = false;
		Generation = InGeneration;
	}

	UPROPERTY()
	TObjectPtr<AZoneGraphData> ZoneGraphData = nullptr;

	int32 Generation = 1;	// Starting at generation 1 so that 0 can be invalid.
	bool bInUse = false;	// Extra bit indicating that the data is in meant to be in use. This tried to capture the case where ZoneGraphData might get nullified without notifying.
};


UCLASS(MinimalAPI)
class UZoneGraphSubsystem : public UTickableWorldSubsystem
{
	GENERATED_BODY()
public:
	ZONEGRAPH_API UZoneGraphSubsystem();

	ZONEGRAPH_API FZoneGraphDataHandle RegisterZoneGraphData(AZoneGraphData& InZoneGraphData);
	ZONEGRAPH_API void UnregisterZoneGraphData(AZoneGraphData& InZoneGraphData);
	TConstArrayView<FRegisteredZoneGraphData> GetRegisteredZoneGraphData() const { return RegisteredZoneGraphData; }

#if WITH_EDITOR
	FZoneGraphBuilder& GetBuilder() { return Builder; }
#endif

	// Queries

	// Returns Zone Graph data associated with specified handle.
	ZONEGRAPH_API const AZoneGraphData* GetZoneGraphData(const FZoneGraphDataHandle DataHandle) const;

	// Returns Zone Graph data storage associated with specified handle, or nullptr if not found.
	const FZoneGraphStorage* GetZoneGraphStorage(const FZoneGraphDataHandle DataHandle) const
	{
		if (int32(DataHandle.Index) < RegisteredZoneGraphData.Num() && int32(DataHandle.Generation) == RegisteredZoneGraphData[DataHandle.Index].Generation)
		{
			if (const AZoneGraphData* Data = RegisteredZoneGraphData[DataHandle.Index].ZoneGraphData)
			{
				return &Data->GetStorage();
			}
		}
		return nullptr;
	}

	// Find nearest lane that touches the query bounds. Finds results from all registered ZoneGraph data.
	ZONEGRAPH_API bool FindNearestLane(const FBox& QueryBounds, const FZoneGraphTagFilter TagFilter, FZoneGraphLaneLocation& OutLaneLocation, float& OutDistanceSqr) const;

	// Find overlapping lanes that touches the query bounds. Finds results from all registered ZoneGraph data.
	ZONEGRAPH_API bool FindOverlappingLanes(const FBox& QueryBounds, const FZoneGraphTagFilter TagFilter, TArray<FZoneGraphLaneHandle>& OutLanes) const;

	// Find sections of lanes fully overlapping (including lane width). Finds results from all registered ZoneGraph data.
	ZONEGRAPH_API bool FindLaneOverlaps(const FVector& Center, const float Radius, const FZoneGraphTagFilter TagFilter, TArray<FZoneGraphLaneSection>& OutLaneSections) const;
	
	// Moves LaneLocation along a lane.
	ZONEGRAPH_API bool AdvanceLaneLocation(const FZoneGraphLaneLocation& InLaneLocation, const float AdvanceDistance, FZoneGraphLaneLocation& OutLaneLocation) const;

	// Returns point at a distance along a specific lane.
	ZONEGRAPH_API bool CalculateLocationAlongLane(const FZoneGraphLaneHandle LaneHandle, const float Distance, FZoneGraphLaneLocation& OutLaneLocation) const;

	// Find nearest location on a specific lane.
	ZONEGRAPH_API bool FindNearestLocationOnLane(const FZoneGraphLaneHandle LaneHandle, const FBox& Bounds, FZoneGraphLaneLocation& OutLaneLocation, float& OutDistanceSqr) const;

	// Find nearest location on a specific lane.
	ZONEGRAPH_API bool FindNearestLocationOnLane(const FZoneGraphLaneHandle LaneHandle, const FVector& Center, const float Range, FZoneGraphLaneLocation& OutLaneLocation, float& OutDistanceSqr) const;

	// returns true if lane handle is valid.
	ZONEGRAPH_API bool IsLaneValid(const FZoneGraphLaneHandle LaneHandle) const;

	// Returns the length of a specific lane.
	ZONEGRAPH_API bool GetLaneLength(const FZoneGraphLaneHandle LaneHandle, float& OutLength) const;

	// Returns the width of a specific lane.
	ZONEGRAPH_API bool GetLaneWidth(const FZoneGraphLaneHandle LaneHandle, float& OutWidth) const;

	// Returns the Tags of a specific lane.
	ZONEGRAPH_API bool GetLaneTags(const FZoneGraphLaneHandle LaneHandle, FZoneGraphTagMask& OutTags) const;

	// Returns all links to connected lanes of a specific lane.
	ZONEGRAPH_API bool GetLinkedLanes(const FZoneGraphLaneHandle LaneHandle, const EZoneLaneLinkType Types, const EZoneLaneLinkFlags IncludeFlags, const EZoneLaneLinkFlags ExcludeFlags, TArray<FZoneGraphLinkedLane>& OutLinkedLanes) const;

	// Returns handle to first linked lane matching the connection type and flags.
	ZONEGRAPH_API bool GetFirstLinkedLane(const FZoneGraphLaneHandle LaneHandle, const EZoneLaneLinkType Types, const EZoneLaneLinkFlags IncludeFlags, const EZoneLaneLinkFlags ExcludeFlags, FZoneGraphLinkedLane& OutLinkedLane) const;

	// Returns bounds of all ZoneGraph data.
	ZONEGRAPH_API FBox GetCombinedBounds() const;
	
	// Tags
	
	// Returns tag based on name.
	ZONEGRAPH_API FZoneGraphTag GetTagByName(FName TagName) const;

	// Returns the name of a specific tag.
	ZONEGRAPH_API FName GetTagName(FZoneGraphTag Tag) const;

	// Returns info about a specific tag.
	ZONEGRAPH_API const FZoneGraphTagInfo* GetTagInfo(FZoneGraphTag Tag) const;

	// Returns all tag infos
	ZONEGRAPH_API TConstArrayView<FZoneGraphTagInfo> GetTagInfos() const;

#if WITH_EDITOR
	virtual bool IsTickableInEditor() const override { return true; }
#endif /* WITH_EDITOR */

protected:
	ZONEGRAPH_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	ZONEGRAPH_API virtual void PostInitialize() override;
	ZONEGRAPH_API virtual void Deinitialize() override;
	ZONEGRAPH_API virtual void Tick(float DeltaTime) override;
	ZONEGRAPH_API virtual TStatId GetStatId() const override;
	
	ZONEGRAPH_API void RemoveRegisteredDataItem(const int32 Index);
	ZONEGRAPH_API void UnregisterStaleZoneGraphDataInstances();
	ZONEGRAPH_API void RegisterZoneGraphDataInstances();
#if WITH_EDITOR
	ZONEGRAPH_API void OnActorMoved(AActor* Actor);
	ZONEGRAPH_API void OnRequestRebuild();
	ZONEGRAPH_API void SpawnMissingZoneGraphData();

	/** Rebuilds the graph.  
	 * @param bForceRebuild Settings this flag will force rebuild even if the data may be up to date. */
	ZONEGRAPH_API void RebuildGraph(const bool bForceRebuild = false);
#endif // WITH_EDITOR

	FCriticalSection DataRegistrationSection;

	UPROPERTY()
	TArray<FRegisteredZoneGraphData> RegisteredZoneGraphData;
	TArray<int32> ZoneGraphDataFreeList;

	bool bInitialized;

#if WITH_EDITOR
	FDelegateHandle OnActorMovedHandle;
	FDelegateHandle OnRequestRebuildHandle;
	FZoneGraphBuilder Builder;
#endif
};

template<>
struct TMassExternalSubsystemTraits<UZoneGraphSubsystem> final
{
	enum
	{
		GameThreadOnly = false,
		ThreadSafeWrite = false
	};
};
