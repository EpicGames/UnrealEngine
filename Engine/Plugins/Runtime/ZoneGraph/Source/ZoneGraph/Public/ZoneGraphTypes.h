// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ZoneGraphBVTree.h"
#include "Misc/Guid.h"
#include "ZoneGraphTypes.generated.h"

#define UE_API ZONEGRAPH_API

ZONEGRAPH_API DECLARE_LOG_CATEGORY_EXTERN(LogZoneGraph, Warning, All);

USTRUCT()
struct FZoneHandle
{
	GENERATED_BODY()

	static UE_API const FZoneHandle Invalid;

	FZoneHandle() = default;
	explicit FZoneHandle(uint32 InIndex) : Index(InIndex) {}

	bool IsValid() const { return Index != InvalidIndex; }

	bool operator==(const FZoneHandle& RHS) const { return Index == RHS.Index; }
	bool operator!=(const FZoneHandle& RHS) const { return Index != RHS.Index; }

	uint32 GetIndex() const { return Index; }

private:
	static const uint32 InvalidIndex = uint32(-1);

	UPROPERTY()
	uint32 Index = InvalidIndex;
};

UENUM()
enum class EZoneGraphTags
{
	MaxTags = 32,
	MaxTagIndex = MaxTags - 1,
};

USTRUCT(BlueprintType)
struct FZoneGraphTag
{
	GENERATED_BODY()

	static UE_API const FZoneGraphTag None;

	FZoneGraphTag() = default;
	
	explicit FZoneGraphTag(const uint8 InBit)
		: Bit(InBit)
	{
		check(InBit <= uint8(EZoneGraphTags::MaxTagIndex));
	}

	void Set(const uint8 InBit)
	{
		check(InBit <= uint8(EZoneGraphTags::MaxTagIndex));
		Bit = InBit;
	}

	uint8 Get() const
	{
		return Bit;
	}

	void Reset()
	{
		Bit = NoneValue;
	}

	bool IsValid() const
	{
		return Bit != NoneValue;
	}

	bool operator==(const FZoneGraphTag& RHS) const
	{
		return Bit == RHS.Bit;
	}

	bool operator!=(const FZoneGraphTag& RHS) const
	{
		return Bit != RHS.Bit;
	}

	friend uint32 GetTypeHash(const FZoneGraphTag& Tag)
	{
		return Tag.Get();
	}

	friend struct FZoneGraphTagMask;

private:
	static UE_API const uint8 NoneValue;

	UPROPERTY(Category = Zone, EditAnywhere, BlueprintReadWrite, meta = (AllowPrivateAccess = "true"))
	uint8 Bit = NoneValue;
};

UENUM(BlueprintType)
enum class EZoneLaneTagMaskComparison : uint8
{
	Any,		// Does masks share ANY tags.
	All,		// Does masks share ALL tags.
	Not,		// Does NOT masks share any tags.
};

USTRUCT(BlueprintType)
struct FZoneGraphTagMask
{
	GENERATED_BODY()

	static UE_API const FZoneGraphTagMask All;
	static UE_API const FZoneGraphTagMask None;

	FZoneGraphTagMask() = default;
	FZoneGraphTagMask(const FZoneGraphTag InTag) : Mask(InTag.IsValid() ? uint32(1) << InTag.Bit : 0) {}
	explicit FZoneGraphTagMask(uint32 InMask) : Mask(InMask) {}

	void Add(const FZoneGraphTagMask InTags)
	{
		Mask |= InTags.Mask;
	}

	void Add(const FZoneGraphTag InTag)
	{
		if (InTag.IsValid())
		{
			Mask |= (uint32(1) << InTag.Bit);
		}
	}

	void Remove(const FZoneGraphTagMask InTags)
	{
		Mask &= ~InTags.Mask;
	}

	void Remove(const FZoneGraphTag InTag)
	{
		if (InTag.IsValid())
		{
			Mask &= ~(uint32(1) << InTag.Bit);
		}
	}

	bool ContainsAny(const FZoneGraphTagMask InTags) const
	{
		return (Mask & InTags.Mask) != 0;
	}

	bool ContainsAll(const FZoneGraphTagMask InTags) const
	{
		return (Mask & InTags.Mask) == InTags.Mask;
	}

	bool Contains(const FZoneGraphTag InTag) const
	{
		return InTag.IsValid() && (Mask & (uint32(1) << InTag.Bit)) != 0;
	}

	bool CompareMasks(const FZoneGraphTagMask InTags, const EZoneLaneTagMaskComparison Operand) const
	{
		switch(Operand)
		{
		case EZoneLaneTagMaskComparison::All:
			return ContainsAll(InTags);
		case EZoneLaneTagMaskComparison::Any:
			return ContainsAny(InTags);
		case EZoneLaneTagMaskComparison::Not:
			return !ContainsAny(InTags);
		default:
			ensureMsgf(false, TEXT("Unhandled operand %s."), *UEnum::GetValueAsString(Operand));
		}
		return false;
	}

	uint32 GetValue() const { return Mask; }

	bool operator==(const FZoneGraphTagMask& RHS) const { return Mask == RHS.Mask; }
	bool operator!=(const FZoneGraphTagMask& RHS) const { return Mask != RHS.Mask; }

	friend uint32 GetTypeHash(const FZoneGraphTagMask& ZoneGraphTagMask)
	{
		return ZoneGraphTagMask.GetValue();
	}

	friend FZoneGraphTagMask operator&(const FZoneGraphTagMask& LHS, const FZoneGraphTagMask& RHS) { return FZoneGraphTagMask(LHS.Mask & RHS.Mask); }
	friend FZoneGraphTagMask operator|(const FZoneGraphTagMask& LHS, const FZoneGraphTagMask& RHS) { return FZoneGraphTagMask(LHS.Mask | RHS.Mask); }
	friend FZoneGraphTagMask operator~(const FZoneGraphTagMask& InMask) { return FZoneGraphTagMask(~InMask.Mask); }

	friend struct FZoneGraphTag;

private:
	UPROPERTY(Category = Zone, EditAnywhere) // BlueprintReadWrite, meta = (AllowPrivateAccess = "true"))  Blueprint does not understand uint32
	uint32 Mask = 0;
};

// Filter passes if any of the 'AnyTags', and all of the 'AllTags', and none of the 'NotTags' are present.
// Setting include or exclude tags to None, will skip that particular check.
USTRUCT()
struct FZoneGraphTagFilter
{
	GENERATED_BODY()

	bool Pass(const FZoneGraphTagMask Tags) const
	{
		return (AnyTags == FZoneGraphTagMask::None || Tags.ContainsAny(AnyTags))
				&& (AllTags == FZoneGraphTagMask::None || Tags.ContainsAll(AllTags))
				&& (NotTags == FZoneGraphTagMask::None || !Tags.ContainsAny(NotTags));
	}

	bool operator==(const FZoneGraphTagFilter& RHS) const { return AnyTags == RHS.AnyTags && AllTags == RHS.AllTags && NotTags == RHS.NotTags; }
	bool operator!=(const FZoneGraphTagFilter& RHS) const { return AnyTags != RHS.AnyTags || AllTags != RHS.AllTags || NotTags != RHS.NotTags; }

	UPROPERTY(Category = Zone, EditAnywhere)
	FZoneGraphTagMask AnyTags = FZoneGraphTagMask::None;

	UPROPERTY(Category = Zone, EditAnywhere)
	FZoneGraphTagMask AllTags = FZoneGraphTagMask::None;

	UPROPERTY(Category = Zone, EditAnywhere)
	FZoneGraphTagMask NotTags = FZoneGraphTagMask::None;
};

USTRUCT()
struct FZoneGraphTagInfo
{
	GENERATED_BODY()

	bool IsValid() const { return !Name.IsNone(); }

	UPROPERTY(Category = Zone, EditAnywhere)
	FName Name;

	UPROPERTY(Category = Zone, EditAnywhere)
	FColor Color = FColor(ForceInit);

	UPROPERTY(Category = Zone, EditAnywhere)
	FZoneGraphTag Tag;
};

UENUM(BlueprintType)
enum class EZoneLaneDirection : uint8
{
	None = 0x0,			// No movement, this lane is treated as spacer or median.
	Forward = 0x1,		// Move forward relative to the markup.
	Backward = 0x2,		// Move backward relative to the markup.
};

// Describes single lane.
USTRUCT(BlueprintType)
struct FZoneLaneDesc
{
	GENERATED_BODY()

	// Width of the lane
	UPROPERTY(Category = Lane, EditAnywhere, BlueprintReadOnly)
	float Width = 150.0f;

	// Direction of the lane
	UPROPERTY(Category = Lane, EditAnywhere, BlueprintReadOnly)
	EZoneLaneDirection Direction = EZoneLaneDirection::Forward;

	// Lane tags
	UPROPERTY(Category = Lane, EditAnywhere, BlueprintReadOnly)
	FZoneGraphTagMask Tags = FZoneGraphTagMask(1);	// Default Tag

	UE_API bool operator==(const FZoneLaneDesc& Other) const;
};

// Describes template of multiple parallel lanes, created in settings.
USTRUCT()
struct FZoneLaneProfile
{
	GENERATED_BODY()

	FZoneLaneProfile()
	: ID(FGuid::NewGuid())
	{
	}

	// Gets combined total with of all lanes.
	float GetLanesTotalWidth() const
	{
		float TotalWidth = 0.0f;
		for (const FZoneLaneDesc& Lane : Lanes)
		{
			TotalWidth += Lane.Width;
		}
		return TotalWidth;
	}

	// Returns true of the profile is symmetrical.
	UE_API bool IsSymmetrical() const;

	// Reverses the lane profile. The lanes array will be reversed, as well as the lane directions. 
	UE_API void ReverseLanes();

	UPROPERTY(Category = Lane, EditAnywhere)
	FName Name;

	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	FGuid ID;

	UPROPERTY(Category = Lane, EditAnywhere)
	TArray<FZoneLaneDesc> Lanes;
};

// Reference to a lane profile.
USTRUCT(BlueprintType)
struct FZoneLaneProfileRef
{
	GENERATED_BODY()

	FZoneLaneProfileRef()
	{
	}

	FZoneLaneProfileRef(const FZoneLaneProfile& LaneProfile)
	{
		Name = LaneProfile.Name;
		ID = LaneProfile.ID;
	}

	bool Equals(const FZoneLaneProfile& LaneProfile) const
	{
		return ID == LaneProfile.ID;
	}

	bool operator==(const FZoneLaneProfileRef& RHS) const
	{
		return ID == RHS.ID;
	}

	UPROPERTY(Category = Lane, EditAnywhere, BlueprintReadWrite)
	FName Name;

	UPROPERTY(Category = Lane, EditAnywhere, BlueprintReadWrite)
	FGuid ID;
};

/** Describes how the linked lane relates to the lane spatially.
	Each type is a bitmask, so that some APIs can filter based on link types. */
UENUM()
enum class EZoneLaneLinkType : uint8
{
	None				= 0,
	All					= MAX_uint8,
	Outgoing			= 1 << 0,	// The lane is connected at the end of the current lane and going out.
	Incoming			= 1 << 1,	// The lane is connected at the beginning of the current lane and coming in.
	Adjacent			= 1 << 2,	// The lane is in same zone, immediately adjacent to the current lane, can be opposite direction or not, see EZoneLaneLinkFlags
};
ENUM_CLASS_FLAGS(EZoneLaneLinkType)

/** Flags describing the details of a linked adjacent lane.
	Flags are only used for adjacent lanes. */
UENUM()
enum class EZoneLaneLinkFlags : uint8
{
	None				= 0,
	All					= MAX_uint8,
	Left				= 1 << 0,	// Left of the current lane
	Right				= 1 << 1,	// Right of the current lane
	Splitting			= 1 << 2,	// Splitting from current lane at start
	Merging				= 1 << 3,	// Merging into the current lane at end
	OppositeDirection	= 1 << 4,	// Opposition direction than current lane
};
ENUM_CLASS_FLAGS(EZoneLaneLinkFlags)

USTRUCT()
struct FZoneLaneLinkData
{
	GENERATED_BODY()

	FZoneLaneLinkData() = default;
	FZoneLaneLinkData(const int32 InDestLaneIndex, const EZoneLaneLinkType InType, const EZoneLaneLinkFlags InFlags) : DestLaneIndex(InDestLaneIndex), Type(InType), Flags((uint8)InFlags) {}

	bool HasFlags(const EZoneLaneLinkFlags InFlags) const { return (Flags & (uint8)InFlags) != 0; }
	EZoneLaneLinkFlags GetFlags() const { return (EZoneLaneLinkFlags)Flags; }
	void SetFlags(const EZoneLaneLinkFlags InFlags) { Flags = (uint8)InFlags; }
	
	/** Index to destination lane in FZoneGraphStorage::Lanes. */
	UPROPERTY()
	int32 DestLaneIndex = 0;

	/** Type of the connection. */
	UPROPERTY()
	EZoneLaneLinkType Type = EZoneLaneLinkType::None;

	/** Specifics about the connection type, see EZoneLaneLinkFlags. */
	UPROPERTY()
	uint8 Flags = 0;
};

// TODO: We could replace *End with *Num, and use uint16. Begin probably needs to be int32/uint32
USTRUCT()
struct FZoneLaneData
{
	GENERATED_BODY()

	int32 GetLinkCount() const { return LinksEnd - LinksBegin; }
	int32 GetNumPoints() const { return PointsEnd - PointsBegin; }
	int32 GetLastPoint() const { return PointsEnd - 1; }

	// Width of the lane
	UPROPERTY()
	float Width = 0.0f;

	// Lane tags
	UPROPERTY()
	FZoneGraphTagMask Tags = FZoneGraphTagMask(1);	// Default Tag

	// First point of the lane polyline in FZoneGraphStorage::LanePoints.
	UPROPERTY()
	int32 PointsBegin = 0;

	// One past the last point of the lane polyline.
	UPROPERTY()
	int32 PointsEnd = 0;

	// First link in FZoneGraphStorage::LaneLinks.
	UPROPERTY()
	int32 LinksBegin = 0;

	// One past the last lane link.
	UPROPERTY()
	int32 LinksEnd = 0;

	// Index of the zone this lane belongs to.
	UPROPERTY()
	int32 ZoneIndex = 0;

	// Source data entry ID, this generally corresponds to input data point index.
	UPROPERTY()
	uint16 StartEntryId = 0;

	// Source data entry ID.
	UPROPERTY()
	uint16 EndEntryId = 0;
};

USTRUCT()
struct FZoneData
{
	GENERATED_BODY()

	int32 GetLaneCount() const { return LanesEnd - LanesBegin; }

	// First point of the zone boundary polyline in FZoneGraphStorage::BoundaryPoints.
	UPROPERTY()
	int32 BoundaryPointsBegin = 0;

	// One past the last point of the zone boundary polyline.
	UPROPERTY()
	int32 BoundaryPointsEnd = 0;

	// First lane of the zone in FZoneGraphStorage::Lanes.
	UPROPERTY()
	int32 LanesBegin = 0;

	// One past the last lane.
	UPROPERTY()
	int32 LanesEnd = 0;

	// Bounding box of the zone
	UPROPERTY()
	FBox Bounds = FBox(ForceInit);

	// Zone tags
	UPROPERTY()
	FZoneGraphTagMask Tags = FZoneGraphTagMask(1);	// Default Tag
};

USTRUCT()
struct FZoneGraphDataHandle
{
	GENERATED_BODY()

	static UE_API const uint16 InvalidGeneration;	// 0

	FZoneGraphDataHandle() = default;
	FZoneGraphDataHandle(const uint16 InIndex, const uint16 InGeneration) : Index(InIndex), Generation(InGeneration) {}

	UPROPERTY(Transient)
	uint16 Index = 0;

	UPROPERTY(Transient)
	uint16 Generation = 0;

	bool operator==(const FZoneGraphDataHandle& Other) const
	{
		return Index == Other.Index && Generation == Other.Generation;
	}

	bool operator!=(const FZoneGraphDataHandle& Other) const
	{
		return !operator==(Other);
	}

	friend uint32 GetTypeHash(const FZoneGraphDataHandle& Handle)
	{
		return uint32(Handle.Index) | (uint32(Handle.Generation) << 16);
	}

	void Reset()
	{
		Index = 0;
		Generation = InvalidGeneration;
	}

	bool IsValid() const { return Generation != InvalidGeneration; }	// Any index is valid, but Generation = 0 means invalid.
};

USTRUCT()
struct FZoneGraphLaneHandle
{
	GENERATED_BODY()

	FZoneGraphLaneHandle() = default;
	FZoneGraphLaneHandle(const int32 InLaneIndex, const FZoneGraphDataHandle InDataHandle) : Index(InLaneIndex), DataHandle(InDataHandle) {}

	UPROPERTY(Transient)
	int32 Index = INDEX_NONE;

	UPROPERTY(Transient)
	FZoneGraphDataHandle DataHandle;

	bool operator==(const FZoneGraphLaneHandle& Other) const
	{
		return Index == Other.Index && DataHandle == Other.DataHandle;
	}

	bool operator!=(const FZoneGraphLaneHandle& Other) const
	{
		return !operator==(Other);
	}

	friend uint32 GetTypeHash(const FZoneGraphLaneHandle& Handle)
	{
		return HashCombine(Handle.Index, GetTypeHash(Handle.DataHandle));
	}

	void Reset()
	{
		Index = INDEX_NONE;
		DataHandle.Reset();
	}

	FString ToString() const { return FString::Printf(TEXT("[%d/%d]"), DataHandle.Index, Index); }
	
	bool IsValid() const { return (Index != INDEX_NONE) && DataHandle.IsValid(); }
};

USTRUCT()
struct FZoneGraphLaneLocation
{
	GENERATED_BODY()
		
	UPROPERTY(Transient)
	FVector Position = FVector::ZeroVector;

	UPROPERTY(Transient)
	FVector Direction = FVector::ForwardVector;

	UPROPERTY(Transient)
	FVector Tangent = FVector::ForwardVector;

	UPROPERTY(Transient)
	FVector Up = FVector::UpVector;

	UPROPERTY(Transient)
	FZoneGraphLaneHandle LaneHandle;

	UPROPERTY(Transient)
	int32 LaneSegment = 0;

	UPROPERTY(Transient)
	float DistanceAlongLane = 0.0f;

	void Reset()
	{
		Position = FVector::ZeroVector;
		Direction = FVector::ForwardVector;
		Tangent = FVector::ForwardVector;
		Up = FVector::UpVector;
		LaneHandle.Reset();
		LaneSegment = 0;
		DistanceAlongLane = 0.0f;
	}

	bool IsValid() const { return LaneHandle.IsValid(); }
};

/** Minimal amount of data to search and compare lane location. */
USTRUCT()
struct FZoneGraphCompactLaneLocation
{
	GENERATED_BODY()

	FZoneGraphCompactLaneLocation() = default;
	FZoneGraphCompactLaneLocation(const FZoneGraphLaneLocation& Location) : LaneHandle(Location.LaneHandle), DistanceAlongLane(Location.DistanceAlongLane) {}
	FZoneGraphCompactLaneLocation(const FZoneGraphLaneHandle Handle, const float Distance) : LaneHandle(Handle), DistanceAlongLane(Distance) {}

	UPROPERTY(Transient)
	FZoneGraphLaneHandle LaneHandle;

	UPROPERTY(Transient)
	float DistanceAlongLane = 0.0f;
};

/** Section of a lane */
USTRUCT()
struct FZoneGraphLaneSection
{
	GENERATED_BODY()

	bool operator==(const FZoneGraphLaneSection& Other) const
	{
		return LaneHandle == Other.LaneHandle && StartDistanceAlongLane == Other.StartDistanceAlongLane && EndDistanceAlongLane == Other.EndDistanceAlongLane;
	}
	
	UPROPERTY(Transient)
	FZoneGraphLaneHandle LaneHandle;

	UPROPERTY(Transient)
	float StartDistanceAlongLane = 0.0f;

	UPROPERTY(Transient)
	float EndDistanceAlongLane = 0.0f;
};

/** Linked lane, used for query results. See also: FZoneLaneLinkData */
USTRUCT()
struct FZoneGraphLinkedLane
{
	GENERATED_BODY()

	FZoneGraphLinkedLane() = default;
	FZoneGraphLinkedLane(const FZoneGraphLaneHandle InDestLane, const EZoneLaneLinkType InType, const EZoneLaneLinkFlags InFlags) : DestLane(InDestLane), Type(InType), Flags((uint8)InFlags) {}

	void Reset()
	{
		DestLane.Reset();
		Type = EZoneLaneLinkType::None;
		Flags = 0;
	}

	bool IsValid() const { return DestLane.IsValid(); }

	bool HasFlags(const EZoneLaneLinkFlags InFlags) const { return (Flags & (uint8)InFlags) != 0; }
	EZoneLaneLinkFlags GetFlags() const { return (EZoneLaneLinkFlags)Flags; }
	void SetFlags(const EZoneLaneLinkFlags InFlags) { Flags = (uint8)InFlags; }
	
	/** Destination lane handle */
	UPROPERTY()
	FZoneGraphLaneHandle DestLane = {};

	/** Type of the connection. */
	UPROPERTY()
	EZoneLaneLinkType Type = EZoneLaneLinkType::None;

	/** Specifics about the connection type, see EZoneLaneLinkFlags. */
	UPROPERTY()
	uint8 Flags = 0;
};

struct FZoneGraphLanePath
{
	FZoneGraphLaneLocation StartLaneLocation;
	FZoneGraphLaneLocation EndLaneLocation;
	TArray<FZoneGraphLaneHandle> Lanes;

	void Reset(const TArray<FZoneGraphLaneHandle>::SizeType NewSize = 0)
	{
		StartLaneLocation.Reset();
		EndLaneLocation.Reset();
		Lanes.Reset(NewSize);
	}

	void Add(const FZoneGraphLaneHandle& LaneHandle)
	{
		Lanes.Add(LaneHandle);
	}
};

USTRUCT()
struct FZoneGraphStorage
{
	GENERATED_BODY()

	UE_API void Reset();
	const FZoneData& GetZoneDataFromLaneIndex(int32 LaneIndex) const { return Zones[Lanes[LaneIndex].ZoneIndex]; }

	// All the zones.
	UPROPERTY()
	TArray<FZoneData> Zones;

	// All the lanes, referred by zones.
	UPROPERTY()
	TArray<FZoneLaneData> Lanes;

	// All the zone boundary points, referred by zones.
	UPROPERTY()
	TArray<FVector> BoundaryPoints;

	// All the lane points, referred by lanes.
	UPROPERTY()
	TArray<FVector> LanePoints;

	// All the lane up vectors, referred by lanes.
	UPROPERTY()
	TArray<FVector> LaneUpVectors;

	// All the lane tangent vectors, referred by lanes.
	UPROPERTY()
	TArray<FVector> LaneTangentVectors;

	// All the lane progression distances, referred by lanes.
	UPROPERTY()
	TArray<float> LanePointProgressions;

	// All the lane links, referred by lanes.
	UPROPERTY()
	TArray<FZoneLaneLinkData> LaneLinks;

	// Bounding box of all zones.
	UPROPERTY()
	FBox Bounds = FBox(ForceInit);

	// BV-Tree of Zones
	UPROPERTY()
	FZoneGraphBVTree ZoneBVTree;
	
	// The handle that this storage represents, updated when data is registered to ZoneGraphSubsystem, used for query results.
	FZoneGraphDataHandle DataHandle;
};


UENUM(BlueprintType)
enum class FZoneShapeType : uint8
{
	Spline,			// Bezier spline shape
	Polygon,		// Bezier polygon shape
};

UENUM(BlueprintType)
enum class EZoneShapePolygonRoutingType : uint8
{
	Bezier,			// Use bezier curves for routing.
	Arcs,			// Use arcs for lane routing.
};

UENUM(BlueprintType, meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EZoneShapeLaneConnectionRestrictions : uint8
{
	None = 0 UMETA(Hidden),
	NoLeftTurn = 1 << 0,						// No left turning destinations allowed.
	NoRightTurn = 1 << 1,						// No right turning destinations allowed.
	OneLanePerDestination = 1 << 2,				// Connect to only one nearest lane per destination.
	MergeLanesToOneDestinationLane = 1 << 3,	// Connect all the incoming lanes to one destination lane.
};
ENUM_CLASS_FLAGS(EZoneShapeLaneConnectionRestrictions)

UENUM(BlueprintType)
enum class FZoneShapePointType : uint8
{
	Sharp,			// Sharp corner
	Bezier,			// Round corner, defined by manual bezier handles
	AutoBezier,		// Round corner, defined by automatic bezier handles
	LaneProfile,	// Lane profile corner, length is defined by lane profile.
};

USTRUCT(BlueprintType)
struct FZoneShapePoint
{
	GENERATED_BODY()

	static UE_API const uint8 InheritLaneProfile;

	FZoneShapePoint() : Type(FZoneShapePointType::Sharp), LaneProfile(InheritLaneProfile) {}
	FZoneShapePoint(const FVector& InPosition) : Position(InPosition), Type(FZoneShapePointType::Sharp), LaneProfile(InheritLaneProfile) {}

	/** Returns incoming Bezier control point. Adjust rotation. */
	UE_API FVector GetInControlPoint() const;

	/** Returns outgoing Bezier control point. Adjust rotation. */
	UE_API FVector GetOutControlPoint() const;

	/** Sets incoming Bezier control point. Adjust rotation. */
	UE_API void SetInControlPoint(const FVector& InPoint);

	/** Sets outgoing Bezier control point. Adjust rotation. */
	UE_API void SetOutControlPoint(const FVector& InPoint);

	/** Returns left edge of lane profile control point (similar relation as outgoing control point). Adjust rotation. */
	UE_API FVector GetLaneProfileLeft() const;
	
	/** Returns right edge of lane profile control point (similar relation as incoming control point). Adjust rotation. */
	UE_API FVector GetLaneProfileRight() const;

	/** Sets left edge of lane profile control point. */
	UE_API void SetLaneProfileLeft(const FVector& InPoint);
	
	/** Returns right edge of lane profile control point. Adjust rotation. */
	UE_API void SetLaneProfileRight(const FVector& InPoint);

	/** Sets rotation pitch/yaw to match Forward direction, and then uses Up direction to find roll angle. */
	UE_API void SetRotationFromForwardAndUp(const FVector& Forward, const FVector& Up);

	void SetLaneConnectionRestrictions(const EZoneShapeLaneConnectionRestrictions Restrictions) { LaneConnectionRestrictions = int32(Restrictions); }
	EZoneShapeLaneConnectionRestrictions GetLaneConnectionRestrictions() const { return EZoneShapeLaneConnectionRestrictions(LaneConnectionRestrictions); }
	
	/** Position of the point */
	UPROPERTY(Category = Zone, EditAnywhere, BlueprintReadWrite)
	FVector Position = FVector::ZeroVector;

	/** Incoming control point */
	UPROPERTY()
	FVector InControlPoint_DEPRECATED = FVector::ZeroVector;

	/** Outgoing control point */
	UPROPERTY()
	FVector OutControlPoint_DEPRECATED = FVector::ZeroVector;

	/** Length of the Bezier point tangents, or cached half-width of the lane profile. */
	UPROPERTY(Category = Zone, EditAnywhere, BlueprintReadWrite)
	float TangentLength = 0.0f;

	/** Inner turn radius associated with this point. Used when polygon shape routing is set to 'Arcs'. */
	UPROPERTY(Category = Zone, EditAnywhere, BlueprintReadWrite)
	float InnerTurnRadius = 100.0f;

	/** Rotation of the point. Forward direction of the rotation matches the tangents. 
	  * For Lane Profile points, the forward directions points into the shape so that we can match the incoming lanes rotation. */
	UPROPERTY(Category = Zone, EditAnywhere, BlueprintReadWrite)
	FRotator Rotation = FRotator::ZeroRotator;

	/** Type of the control point */
	UPROPERTY(Category = Zone, EditAnywhere, BlueprintReadWrite)
	FZoneShapePointType Type = FZoneShapePointType::Sharp;

	/** Index to external array referring to Lane Profile, or FZoneShapePoint::InheritLaneProfile if we should use Shape's lane profile.
	  * This is a little awkward indirection, but keeps the point memory usage in check. */
	UPROPERTY(Category = Zone, EditAnywhere, BlueprintReadWrite)
	uint8 LaneProfile = InheritLaneProfile;

	/** True of lane profile should be reversed. */
	UPROPERTY(Category = Zone, EditAnywhere, BlueprintReadWrite)
	bool bReverseLaneProfile = false;

	/** Lane connection restrictions */
	UPROPERTY(Category = Zone, EditAnywhere, BlueprintReadWrite, meta = (Bitmask, BitmaskEnum = "/Script/ZoneGraph.EZoneShapeLaneConnectionRestrictions"))
	int32 LaneConnectionRestrictions = 0;
};

// Shape connectors represent locations where shapes can be connected together.
USTRUCT()
struct FZoneShapeConnector
{
	GENERATED_BODY()

	// Position of the connector.
	UPROPERTY()
	FVector Position = FVector::ZeroVector;

	// Normal direction of the connector.
	UPROPERTY()
	FVector Normal = FVector::ForwardVector;

	// Up direction of the connector.
	UPROPERTY()
	FVector Up = FVector::UpVector;

	// Point index of UZoneShapeComponent.
	UPROPERTY()
	int32 PointIndex = 0;

	// Lane template of the connector.
	UPROPERTY()
	FZoneLaneProfileRef LaneProfile;

	// True if lane profile should be treated as reversed.
	UPROPERTY()
	bool bReverseLaneProfile = false;

	// Which type of shape the connector belongs to.
	UPROPERTY()
	FZoneShapeType ShapeType = FZoneShapeType::Spline;
};

// Connection between two shape connectors.
USTRUCT()
struct FZoneShapeConnection
{
	GENERATED_BODY()

	// Connected shape.
	UPROPERTY()
	TWeakObjectPtr<class UZoneShapeComponent> ShapeComponent = nullptr;

	// Connector index at the connected shape.
	UPROPERTY()
	int32 ConnectorIndex = 0;
};


USTRUCT()
struct FZoneGraphTessellationSettings
{
	GENERATED_BODY()

	// Lanes to apply this tolerance
	UPROPERTY(Category = Zone, EditAnywhere)
	FZoneGraphTagFilter LaneFilter;

	// Tessellation tolerance, the error between tessellated point and the spline.
	UPROPERTY(Category = Zone, EditAnywhere)
	float TessellationTolerance = 1.0f;
};

UENUM(BlueprintType)
enum class EZoneGraphLaneRoutingCountRule : uint8
{
	Any,			// Any number of entries
	One,			// Just one entry
	Many,			// Many entries
};

USTRUCT()
struct FZoneGraphLaneRoutingRule
{
	GENERATED_BODY()

	UPROPERTY(Category = Rule, EditAnywhere)
	bool bEnabled = true;

	UPROPERTY(Category = Rule, EditAnywhere)
	FString Comment;

	UPROPERTY(Category = Rule, EditAnywhere)
	FZoneGraphTagFilter ZoneTagFilter;

	UPROPERTY(Category = Rule, EditAnywhere)
	FZoneLaneProfileRef SourceLaneProfile;

	UPROPERTY(Category = Rule, EditAnywhere)
	FZoneLaneProfileRef DestinationLaneProfile;

	UPROPERTY(Category = Rule, EditAnywhere)
	EZoneGraphLaneRoutingCountRule SourceOutgoingConnections = EZoneGraphLaneRoutingCountRule::Any;

	UPROPERTY(Category = Rule, EditAnywhere)
	EZoneGraphLaneRoutingCountRule DestinationIncomingConnections = EZoneGraphLaneRoutingCountRule::Any;

	UPROPERTY(Category = Rule, EditAnywhere, meta = (Bitmask, BitmaskEnum = "/Script/ZoneGraph.EZoneShapeLaneConnectionRestrictions"))
	int32 ConnectionRestrictions = 0;
};

USTRUCT()
struct FZoneGraphBuildSettings
{
	GENERATED_BODY()
		
	/** @return tessellation tolerance for specific case, or common tolerance if no match. */
	UE_API float GetLaneTessellationTolerance(const FZoneGraphTagMask LaneTags) const;

	/** @return Connection restrictions for specified lane based on PolygonRoutingRules */
	UE_API EZoneShapeLaneConnectionRestrictions GetConnectionRestrictions(const FZoneGraphTagMask ZoneTags,
																   const FZoneLaneProfileRef& SourceLaneProfile, const int32 SourceConnectionCount,
																   const FZoneLaneProfileRef& DestinationLaneProfile, const int32 DestinationConnectionCount) const;
	
	/** Common tolerance for all lane tessellation, the error between tessellated point and the spline. */
	UPROPERTY(Category = Lanes, EditAnywhere)
	float CommonTessellationTolerance = 1.0f;

	/** Custom tessellation tolerances based on lane tags, first match is returned. */
	UPROPERTY(Category = Lanes, EditAnywhere)
	TArray<FZoneGraphTessellationSettings> SpecificTessellationTolerances;

	/** Max relative angle (in degrees) between two lane profiles for them to be connected with lanes. In degrees. */
	UPROPERTY(Category = Lanes, EditAnywhere)
	float LaneConnectionAngle = 120.0f;

	/** Mask of tags which should be used to check if lanes should connect. */
	UPROPERTY(Category = Lanes, EditAnywhere)
	FZoneGraphTagMask LaneConnectionMask = FZoneGraphTagMask::All;

	/** When the relative angle (in degrees) to destination on a polygon is more than the specified angle, it is considered left or right turn. */
	UPROPERTY(Category = Lanes, EditAnywhere)
	float TurnThresholdAngle = 5.0f;

	/** Routing rules applied to polygon shapes */
	UPROPERTY(Category = Lanes, EditAnywhere)
	TArray<FZoneGraphLaneRoutingRule> PolygonRoutingRules;

	/** Max distance between two shape points for them to be snapped together. */
	UPROPERTY(Category = PointSnapping, EditAnywhere)
	float ConnectionSnapDistance = 25.0f;

	/** Max relative angle (in degrees) between two shape points for them to be snapped together. */
	UPROPERTY(Category = PointSnapping, EditAnywhere)
	float ConnectionSnapAngle = 10.0f;

	/** Max distance to auto connect a dragged end control point to the closest connector when released. */
	UPROPERTY(Category = PointSnapping, EditAnywhere)
	double DragEndpointAutoConnectRange = 250.0;

	/** Max distance to automatically create an intersection with a dragged end control point and the overlapping zone shape when released. */
	UPROPERTY(Category = PointSnapping, EditAnywhere)
	double DragEndpointAutoIntersectionRange = 250.0;

	/** If the location to create auto intersection is closer than the tolerance to a spline point, use the spline point instead of nearest point on spline to create the intersection. */
	UPROPERTY(Category = PointSnapping, EditAnywhere)
	double SnapAutoIntersectionToClosestPointTolerance = 100.0;

	/** If true, draws a sphere for the auto connection/interseciont area. Otherwise, draws a circle. */
	UPROPERTY(Category = PointSnapping, EditAnywhere)
	bool bShow3DRadiusForAutoConnectionAndIntersection = false;
};

#undef UE_API
