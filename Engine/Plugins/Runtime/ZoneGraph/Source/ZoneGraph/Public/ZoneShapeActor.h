// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "ZoneShapeActor.generated.h"

#define UE_API ZONEGRAPH_API

class UZoneShapeComponent;

/** Zone Shape actor for standalone zone markup. */
UCLASS(MinimalAPI, hidecategories = (Input))
class AZoneShape : public AActor
{
	GENERATED_BODY()
public:
	UE_API AZoneShape(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual bool IsEditorOnly() const override { return true; }

	const UZoneShapeComponent* GetShape() const { return ShapeComponent; }

#if WITH_EDITOR
	virtual bool CanChangeIsSpatiallyLoadedFlag() const override { return false; }
#endif

protected:

	UPROPERTY(Category = Zone, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UZoneShapeComponent> ShapeComponent;
};

#undef UE_API
