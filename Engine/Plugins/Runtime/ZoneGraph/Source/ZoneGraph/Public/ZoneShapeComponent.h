// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/PrimitiveComponent.h"
#include "ZoneGraphSubsystem.h"
#include "ZoneGraphTypes.h"
#include "ZoneShapeComponent.generated.h"

#define UE_API ZONEGRAPH_API


/** Custom serialization version for ZoneShapeComponent */
struct FZoneShapeCustomVersion
{
	enum Type
	{
		// Before any version changes were made in the plugin
		BeforeCustomVersionWasAdded = 0,
		// Added roll to points.
		AddedRoll,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	/** The GUID for this custom version number */
	UE_API const static FGuid GUID;

private:
	FZoneShapeCustomVersion() {}
};


UCLASS(MinimalAPI, ClassGroup = Custom, BlueprintType, ShowCategories = (Mobility), HideCategories = (Physics, Collision, Lighting, Rendering, Mobile), meta = (BlueprintSpawnableComponent))
class UZoneShapeComponent : public UPrimitiveComponent
{
	GENERATED_BODY()
public:
	UE_API UZoneShapeComponent(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	virtual bool IsEditorOnly() const override { return true; }

	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;

	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
#if WITH_EDITOR
	DECLARE_EVENT(UZoneShapeComponent, FOnShapeDataChanged);
	FOnShapeDataChanged& OnShapeDataChanged() { return ShapeDataChangedEvent; }

	UE_API virtual void PostEditImport() override;
	UE_API virtual void PostEditUndo() override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;

#if !UE_BUILD_SHIPPING
	//~ Begin UPrimitiveComponent Interface.
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	//~ End UPrimitiveComponent Interface.
#endif

	/** Updates shape, calculates auto tangents, and adjusts lane points to match the lane templates, updates connectors and connections. */
	UE_API void UpdateShape();

	/** Updates shape connectors from the points. */
	UE_API void UpdateShapeConnectors();

	/** Updates how shape connectors are connected to other shapes. */
	UE_API void UpdateConnectedShapes();

	/** Updates how shape connectors are connected to other shapes. In case a connection is found, the mating shape is updated too. */
	UE_API void UpdateMatingConnectedShapes();

	/** Calculates rotation and tangent for a specific point.
	 * @param PointIndex index of the point to update
	 */
	UE_API void UpdatePointRotationAndTangent(int32 PointIndex);

	/** Returns true if the shape is closed form. */
	UE_API bool IsShapeClosed() const;

	/** Adds new one if one does not exists
	 * @param NewLaneProfileRef New profile to add.
	 * @return index to the lane profile, or INDEX_NONE if new lane template could not be added.
	 */
	UE_API int32 AddUniquePerPointLaneProfile(const FZoneLaneProfileRef& NewLaneProfileRef);

	/** Removes unused items from the per point lane template array. */
	UE_API void CompactPerPointLaneProfiles();

	/** Removes per point lane templates. */
	UE_API void ClearPerPointLaneProfiles();

	/** @return The shape template used for the spline shape. */
	UE_API void GetSplineLaneProfile(FZoneLaneProfile& OutLaneProfile) const;

	/** @return Lane template for each point in the polygon shape. */
	UE_API void GetPolygonLaneProfiles(TArray<FZoneLaneProfile>& OutLaneProfiles) const;

	/** @return Lane profile which is used for spline lanes, and polygon points which are set to Inherit. */
	const FZoneLaneProfileRef& GetCommonLaneProfile() { return LaneProfile; }

	/** Sets lane profile which is used for spline lanes, and polygon points which are set to Inherit.
	 * @param LaneProfileRef New lane profile.
	 */
	void SetCommonLaneProfile(const FZoneLaneProfileRef& LaneProfileRef) { LaneProfile = LaneProfileRef; }

	/** @return Number of points in the shape. */
	int32 GetNumPoints() const { return Points.Num(); }

	/** @return Number of segments in the shape. */
	int32 GetNumSegments() const { return Points.Num() - (IsShapeClosed() ? 0 : 1); }

	/** @return View to the points array. */
	TConstArrayView<FZoneShapePoint> GetPoints() const { return Points; }

	/** @return Mutable reference to the points array. */
	TArray<FZoneShapePoint>& GetMutablePoints() { return Points; }

	/** @return lane templates referred by points. */
	TConstArrayView<FZoneLaneProfileRef> GetPerPointLaneProfiles() const { return PerPointLaneProfiles; };

	/** @return View to shape connectors array. */
	TConstArrayView<FZoneShapeConnector> GetShapeConnectors() const { return ShapeConnectors; }

	/** @return View to connected shapes. */
	TConstArrayView<FZoneShapeConnection> GetConnectedShapes() const { return ConnectedShapes; }

	/** @return Connector at the given point. */
	UE_API const FZoneShapeConnector* GetShapeConnectorByPointIndex(int32 PointIndex) const;

	/** @return Shape type. */
	UFUNCTION(Category = Zone, BlueprintPure)
	FZoneShapeType GetShapeType() const { return ShapeType; }

	/** Sets shape type.
	 * @param Type New shape type.
	 */
	UFUNCTION(Category = Zone, BlueprintCallable)
	void SetShapeType(FZoneShapeType Type) { ShapeType = Type; }

	/** @return Shape's tags. */
	UFUNCTION(Category = Zone, BlueprintPure)
	FZoneGraphTagMask GetTags() const { return Tags; }

	/** @return Tags that can be changed. */
	FZoneGraphTagMask& GetMutableTags() { return Tags; }

	/** Sets shape tags.
	 * @param NewTags New tags to set.
	 */
	UFUNCTION(Category = Zone, BlueprintCallable)
	void SetTags(const FZoneGraphTagMask NewTags) { Tags = NewTags; }

	/** @return True if common lane profile is reversed. */
	UFUNCTION(Category = Zone, BlueprintPure)
	bool IsLaneProfileReversed() const { return bReverseLaneProfile; }

	/** Set whether common lane profile should be reversed.
	 * @param bReverse Reverse state
	 * @return Newly set reversed state.
	 */
	UFUNCTION(Category = Zone, BlueprintCallable)
	bool SetReverseLaneProfile(bool bReverse) { return bReverseLaneProfile = bReverse; }

	/** @return Polygon routing type */
	EZoneShapePolygonRoutingType GetPolygonRoutingType() const { return PolygonRoutingType; }

	/** Sets the polygon routing type. */
	UFUNCTION(Category = Zone, BlueprintCallable)
	void SetPolygonRoutingType(const EZoneShapePolygonRoutingType NewType) { PolygonRoutingType = NewType; }

#if WITH_EDITOR
	/** @return Hash value of the shape data. Can be used to determine if the shape has changed. */
	UE_API uint32 GetShapeHash() const;
#endif

private:

	/** Common lane template for whole shape */
	UPROPERTY(Category = Zone, EditAnywhere, BlueprintReadWrite, meta=(AllowPrivateAccess="true", IncludeInHash))
	FZoneLaneProfileRef LaneProfile;

	/** True if lane profile should be reversed */
	UPROPERTY(Category = Zone, EditAnywhere, BlueprintReadWrite, meta = (AllowPrivateAccess = "true", IncludeInHash))
	bool bReverseLaneProfile = false;

	/** Array of lane templates indexed by the points when the shape is polygon. */
	UPROPERTY(Category = Zone, VisibleAnywhere, meta = (IncludeInHash, EditCondition = "false", EditConditionHides))
	TArray<FZoneLaneProfileRef> PerPointLaneProfiles;

	/** Shape points */
	UPROPERTY(Category = Zone, EditAnywhere, BlueprintReadWrite, meta = (AllowPrivateAccess = "true", IncludeInHash))
	TArray<FZoneShapePoint> Points;

	/** Shape type, spline or polygon */
	UPROPERTY(Category = Zone, EditAnywhere, BlueprintReadWrite, meta = (AllowPrivateAccess = "true", IncludeInHash))
	FZoneShapeType ShapeType;

	/** Polygon shape routing type */
	UPROPERTY(Category = Zone, EditAnywhere, BlueprintReadWrite, meta = (AllowPrivateAccess = "true", IncludeInHash, EditCondition = "ShapeType == FZoneShapeType::Polygon"))
	EZoneShapePolygonRoutingType PolygonRoutingType = EZoneShapePolygonRoutingType::Bezier;
	
	/** Zone tags, the lanes inherit zone tags. */
	UPROPERTY(Category = Zone, EditAnywhere, BlueprintReadWrite, meta = (AllowPrivateAccess = "true", IncludeInHash))
	FZoneGraphTagMask Tags;

	/** Connectors for other shapes (not stored, these are refreshed from points). */
	UPROPERTY(Transient)
	TArray<FZoneShapeConnector> ShapeConnectors;

	/** Array of connections matching ShapeConnectors (not stored, these are refreshed from connectors). */
	UPROPERTY(Transient)
	TArray<FZoneShapeConnection> ConnectedShapes;

#if WITH_EDITOR
	UE_API void OnLaneProfileChanged(const FZoneLaneProfileRef& ChangedLaneProfileRef);

	FDelegateHandle OnLaneProfileChangedHandle;
#endif

#if WITH_EDITORONLY_DATA
	FOnShapeDataChanged ShapeDataChangedEvent;
#endif
};

#undef UE_API
