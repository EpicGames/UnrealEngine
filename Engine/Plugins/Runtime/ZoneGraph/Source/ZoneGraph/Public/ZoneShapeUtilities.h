// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ZoneGraphTypes.h"
#include "ZoneShapeUtilities.generated.h"

/** Struct describing a link for a specified lane, used during building */
USTRUCT()
struct FZoneShapeLaneInternalLink
{
	GENERATED_BODY()

	FZoneShapeLaneInternalLink() = default;
	FZoneShapeLaneInternalLink(const int32 InLaneIdex, const FZoneLaneLinkData InLinkData) : LaneIndex(InLaneIdex), LinkData(InLinkData) {}

	bool operator<(const FZoneShapeLaneInternalLink& RHS) const { return LaneIndex < RHS.LaneIndex; }

	/** Lane index to which the link belongs to */
	UPROPERTY()
	int32 LaneIndex = 0;
	
	/** Link details */
	UPROPERTY()
	FZoneLaneLinkData LinkData = {};
};

namespace UE { namespace ZoneShape { namespace Utilities
{

// Converts spline shape to a zone data.
ZONEGRAPH_API void TessellateSplineShape(TConstArrayView<FZoneShapePoint> Points, const FZoneLaneProfile& LaneProfile, const FZoneGraphTagMask ZoneTags, const FMatrix& LocalToWorld,
										 FZoneGraphStorage& OutZoneStorage, TArray<FZoneShapeLaneInternalLink>& OutInternalLinks);

// Converts polygon shape to a zone data.
ZONEGRAPH_API void TessellatePolygonShape(TConstArrayView<FZoneShapePoint> Points, const EZoneShapePolygonRoutingType RoutingType, TConstArrayView<FZoneLaneProfile> LaneProfiles, const FZoneGraphTagMask ZoneTags, const FMatrix& LocalToWorld,
										  FZoneGraphStorage& OutZoneStorage, TArray<FZoneShapeLaneInternalLink>& OutInternalLinks);

// Returns cubic bezier points for give shape segment. Adjusts end points based on shape point types. 
ZONEGRAPH_API void GetCubicBezierPointsFromShapeSegment(const FZoneShapePoint& StartShapePoint, const FZoneShapePoint& EndShapePoint, const FMatrix& LocalToWorld,
														FVector& OutStartPoint, FVector& OutStartControlPoint, FVector& OutEndControlPoint, FVector& OutEndPoint);

}}} // UE::ZoneShape::Utilities
