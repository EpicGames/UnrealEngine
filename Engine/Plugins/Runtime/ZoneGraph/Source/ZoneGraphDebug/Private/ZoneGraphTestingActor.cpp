// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZoneGraphTestingActor.h"
#include "SceneView.h"
#include "UObject/ConstructorHelpers.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "ZoneGraphSubsystem.h"
#include "ZoneGraphDelegates.h"
#include "ZoneGraphRenderingUtilities.h"
#include "ZoneGraphData.h"
#include "ZoneGraphAStar.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ZoneGraphTestingActor)

UZoneGraphTestingComponent::UZoneGraphTestingComponent(const FObjectInitializer& ObjectInitialize)
	: Super(ObjectInitialize)
{
	SearchExtent = FVector(150.0f);
	AdvanceDistance = 300.0f;

	NearestTestOffset = FVector(300, 0, 0);
}

#if WITH_EDITOR
void UZoneGraphTestingComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	UpdateTests();
}
#endif

void UZoneGraphTestingComponent::OnRegister()
{
	Super::OnRegister();
#if WITH_EDITOR
	// Force to update tests when ever the data changes.
	OnDataChangedHandle = UE::ZoneGraphDelegates::OnZoneGraphDataBuildDone.AddUObject(this, &UZoneGraphTestingComponent::OnZoneGraphDataBuildDone);
#endif
	OnDataAddedHandle = UE::ZoneGraphDelegates::OnPostZoneGraphDataAdded.AddUObject(this, &UZoneGraphTestingComponent::OnZoneGraphDataChanged);
	OnDataRemovedHandle = UE::ZoneGraphDelegates::OnPreZoneGraphDataRemoved.AddUObject(this, &UZoneGraphTestingComponent::OnZoneGraphDataChanged);

	ZoneGraph = UWorld::GetSubsystem<UZoneGraphSubsystem>(GetWorld());

	ExecuteOnEachCustomTest([this](UZoneLaneTest& Test) { Test.SetOwner(this); });

	UpdateTests();
}

void UZoneGraphTestingComponent::OnUnregister()
{
	Super::OnUnregister();
#if WITH_EDITOR
	UE::ZoneGraphDelegates::OnZoneGraphDataBuildDone.Remove(OnDataChangedHandle);
#endif
	UE::ZoneGraphDelegates::OnPostZoneGraphDataAdded.Remove(OnDataAddedHandle);
	UE::ZoneGraphDelegates::OnPreZoneGraphDataRemoved.Remove(OnDataRemovedHandle);
}

void UZoneGraphTestingComponent::OnZoneGraphDataChanged(const AZoneGraphData* ZoneGraphData)
{
	// Only consider valid graph from our world
	if (ZoneGraphData == nullptr || ZoneGraphData->GetWorld() != GetWorld())
	{
		return;
	}

	UpdateTests();
}

const FZoneGraphStorage* UZoneGraphTestingComponent::GetZoneGraphStorage(const FZoneGraphLaneHandle& LaneHandle) const
{
	if (ZoneGraph == nullptr)
	{
		return nullptr;
	}

	const AZoneGraphData* ZoneGraphData = ZoneGraph->GetZoneGraphData(LaneHandle.DataHandle);
	return (ZoneGraphData != nullptr) ? &(ZoneGraphData->GetStorage()) : nullptr;
}

void UZoneGraphTestingComponent::EnableCustomTests()
{
	bCustomTestsDisabled = false;
	ExecuteOnEachCustomTest([this](UZoneLaneTest& Test) { Test.OnLaneLocationUpdated(FZoneGraphLaneLocation(), LaneLocation); });
}

void UZoneGraphTestingComponent::DisableCustomTests()
{
	bCustomTestsDisabled = true;
	ExecuteOnEachCustomTest([this](UZoneLaneTest& Test) { Test.OnLaneLocationUpdated(LaneLocation, FZoneGraphLaneLocation()); });
}

#if WITH_EDITOR
void UZoneGraphTestingComponent::OnZoneGraphDataBuildDone(const struct FZoneGraphBuildData& BuildData)
{
	UpdateTests();
}
#endif

FBoxSphereBounds UZoneGraphTestingComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	const FVector ActorPosition = LocalToWorld.GetTranslation();
	return FBox(ActorPosition - SearchExtent, ActorPosition + SearchExtent);
}

void UZoneGraphTestingComponent::UpdateTests()
{
	if (!ZoneGraph)
	{
		return;
	}

	const FVector ActorPosition = GetOwner()->GetActorLocation();

	// Find nearest
	float DistanceSqr = 0.0f;
	const FZoneGraphLaneLocation PrevLaneLocation = LaneLocation;
	ZoneGraph->FindNearestLane(FBox(ActorPosition - SearchExtent, ActorPosition + SearchExtent), QueryFilter, LaneLocation, DistanceSqr);

	if (!bCustomTestsDisabled)
	{
		ExecuteOnEachCustomTest([PrevLaneLocation, this](UZoneLaneTest& Test) { Test.OnLaneLocationUpdated(PrevLaneLocation, LaneLocation); });
	}

	// Test advance
	NextLaneLocation.Reset();
	float DistanceToGo = AdvanceDistance;
	if (ZoneGraph->AdvanceLaneLocation(LaneLocation, DistanceToGo, NextLaneLocation))
	{
		DistanceToGo -= (NextLaneLocation.DistanceAlongLane - LaneLocation.DistanceAlongLane);
		// If hit the end of a lane, keep going on until we have advanced far enough or hit a dead end.
		while (DistanceToGo > KINDA_SMALL_NUMBER)
		{
			TArray<FZoneGraphLinkedLane> NextLanes;
			if (ZoneGraph->GetLinkedLanes(NextLaneLocation.LaneHandle, EZoneLaneLinkType::Outgoing, EZoneLaneLinkFlags::All, EZoneLaneLinkFlags::None, NextLanes) && NextLanes.Num() > 0)
			{
				// Advance to first link.
				ZoneGraph->CalculateLocationAlongLane(NextLanes[0].DestLane, DistanceToGo, NextLaneLocation);
				DistanceToGo -= NextLaneLocation.DistanceAlongLane;
			}
			else
			{
				break;
			}
		}
	}

	// Test nearest location on specific lane
	ZoneGraph->FindNearestLocationOnLane(LaneLocation.LaneHandle, FBox(ActorPosition + NearestTestOffset - SearchExtent, ActorPosition + NearestTestOffset + SearchExtent), NearestLaneLocation, DistanceSqr);

	if (bDrawLinkedLanes)
	{
		// Find linked lanes
		LinkedLanes.Reset();
		ZoneGraph->GetLinkedLanes(LaneLocation.LaneHandle, EZoneLaneLinkType::All, EZoneLaneLinkFlags::All, EZoneLaneLinkFlags::None, LinkedLanes);
	}

	if (bDrawLanePath)
	{
		// Simple pathfind to to the linked AZoneGraphTestingActor for testing purposes
		UZoneGraphTestingComponent* OtherTestingComp = OtherActor ? Cast<UZoneGraphTestingComponent>(OtherActor->GetComponentByClass(UZoneGraphTestingComponent::StaticClass())) : nullptr;
		if (NearestLaneLocation.IsValid() &&
			OtherTestingComp && OtherTestingComp->NearestLaneLocation.IsValid() &&
			NearestLaneLocation.LaneHandle.DataHandle == OtherTestingComp->NearestLaneLocation.LaneHandle.DataHandle)
		{
			const AZoneGraphData* Data = ZoneGraph->GetZoneGraphData(NearestLaneLocation.LaneHandle.DataHandle);
			if (Data)
			{
				const FZoneGraphStorage& ZoneGraphStorage = Data->GetStorage();
				FZoneGraphAStarWrapper Graph(ZoneGraphStorage);
				FZoneGraphAStar Pathfinder(Graph);
				// @todo: pass FZoneGraphLaneLocation directly to the constructor
				FZoneGraphAStarNode StartNode(NearestLaneLocation.LaneHandle.Index, NearestLaneLocation.Position);
				FZoneGraphAStarNode EndNode(OtherTestingComp->NearestLaneLocation.LaneHandle.Index, OtherTestingComp->NearestLaneLocation.Position);
				FZoneGraphPathFilter PathFilter(ZoneGraphStorage, NearestLaneLocation, OtherTestingComp->NearestLaneLocation, QueryFilter);
				
				// @todo: see if we can return directly a path of lane handles
				TArray<FZoneGraphAStarWrapper::FNodeRef> ResultPath;
				EGraphAStarResult Result = Pathfinder.FindPath(StartNode, EndNode, PathFilter, ResultPath);
				if (Result == SearchSuccess)
				{
					//Store the resulting lanes
					LanePath.Reset(ResultPath.Num());

					LanePath.StartLaneLocation = NearestLaneLocation;
					LanePath.EndLaneLocation = OtherTestingComp->NearestLaneLocation;
					for (FZoneGraphAStarWrapper::FNodeRef Node : ResultPath)
					{
						LanePath.Add(FZoneGraphLaneHandle(Node, NearestLaneLocation.LaneHandle.DataHandle));
					}
				}
			}
		}
	}

	MarkRenderStateDirty();
}



#if !UE_BUILD_SHIPPING
FPrimitiveSceneProxy* UZoneGraphTestingComponent::CreateSceneProxy()
{
	class FZoneGraphTestingSceneProxy final : public FPrimitiveSceneProxy
	{
	public:
		friend class UZoneGraphTestingComponent;

		virtual SIZE_T GetTypeHash() const override
		{
			static size_t UniquePointer;
			return reinterpret_cast<size_t>(&UniquePointer);
		}

		FZoneGraphTestingSceneProxy(const UZoneGraphTestingComponent& InComponent)
			: FPrimitiveSceneProxy(&InComponent)
			, Component(&InComponent)
		{
		}

		virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_ShapeSceneProxy_GetDynamicMeshElements);

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if (VisibilityMap & (1 << ViewIndex))
				{
					const FSceneView* View = Views[ViewIndex];
					FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);

					const FMatrix& LocalToWorld = GetLocalToWorld();

					// Taking into account the min and maximum drawing distance
					const double DistanceSqr = (View->ViewMatrices.GetViewOrigin() - LocalToWorld.GetOrigin()).SizeSquared();
					if (DistanceSqr < FMath::Square(GetMinDrawDistance()) || DistanceSqr > FMath::Square(GetMaxDrawDistance()))
					{
						continue;
					}

					// Draw
					DrawView(PDI);
				}
			}
		}

		virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
		{
			FPrimitiveViewRelevance Result;
			Result.bDrawRelevance = IsShown(View);
			Result.bDynamicRelevance = true;
			Result.bShadowRelevance = IsShadowCast(View);
			Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
			return Result;
		}

		virtual uint32 GetMemoryFootprint(void) const override { return sizeof(*this) + GetAllocatedSize(); }
		uint32 GetAllocatedSize(void) const { return (uint32)FPrimitiveSceneProxy::GetAllocatedSize(); }

		void DrawView(FPrimitiveDrawInterface* PDI) const
		{
 			const FZoneGraphLaneLocation& LaneLoc = Component->LaneLocation;
 			const FZoneGraphLaneLocation& NextLaneLoc = Component->NextLaneLocation;
 			const FZoneGraphLaneLocation& NearestLaneLoc = Component->NearestLaneLocation;

			const FMatrix& LocalToWorld = GetLocalToWorld();
			const FVector Center = LocalToWorld.GetOrigin();
			const FVector Offset(0, 0, 10.0f);
			static constexpr float TickSize = 25.0f;
			static constexpr FColor QueryBoxColor(0, 255, 192);
			static constexpr FColor NearestColor(255, 64, 0);
			static constexpr FColor AdvanceColor(64, 255, 0);
			static constexpr FColor NearestLaneColor(0, 64, 255);

			DrawWireBox(PDI, FBox(Center - Component->SearchExtent, Center + Component->SearchExtent), FColor::Blue, SDPG_World);
			PDI->DrawLine(Center - FVector(TickSize, 0, 0), Center + FVector(25.0f, 0, 0), QueryBoxColor, SDPG_World);
			PDI->DrawLine(Center - FVector(0, TickSize, 0), Center + FVector(0, 25.0f, 0), QueryBoxColor, SDPG_World);
			PDI->DrawLine(Center - FVector(0, 0, TickSize), Center + FVector(0, 0, 25.0f), QueryBoxColor, SDPG_World);

			if (LaneLoc.IsValid())
			{
				PDI->DrawLine(Offset + LaneLoc.Position - LaneLoc.Up * TickSize, Offset + LaneLoc.Position + LaneLoc.Up * TickSize * 3.0f, NearestColor, SDPG_World, 2.0f, 0.01f);
				PDI->DrawLine(Center, Offset + LaneLoc.Position, NearestColor, SDPG_World, 2.0f, 0.01f);
			}
			if (LaneLoc.IsValid() && NextLaneLoc.IsValid())
			{
				PDI->DrawLine(Offset + NextLaneLoc.Position - NextLaneLoc.Up * TickSize, Offset + NextLaneLoc.Position + NextLaneLoc.Up * TickSize * 3.0f, AdvanceColor, SDPG_World, 2.0f, 0.01f);
				PDI->DrawLine(Offset + LaneLoc.Position, Offset + NextLaneLoc.Position, AdvanceColor, SDPG_World, 2.0f, 0.01f);
			}

			PDI->DrawLine(Center, Center + Component->NearestTestOffset, FColor::Black, SDPG_World, 0.0f, 0.01f);
			if (NearestLaneLoc.IsValid())
			{
				PDI->DrawLine(Offset + NearestLaneLoc.Position - NearestLaneLoc.Up * TickSize, Offset + NearestLaneLoc.Position + NearestLaneLoc.Up * TickSize * 3.0f, NearestLaneColor, SDPG_World, 2.0f, 0.01f);
				PDI->DrawLine(Center + Component->NearestTestOffset, Offset + NearestLaneLoc.Position, NearestLaneColor, SDPG_World, 2.0f, 0.01f);
			}

			if (LaneLoc.IsValid())
			{
				const FZoneGraphStorage* Storage = Component->GetZoneGraphStorage(LaneLoc.LaneHandle);
				if (Storage != nullptr)
				{
					// Drawing linked lanes from LaneLocation
					if (Component->bDrawLinkedLanes)
					{
						UE::ZoneGraph::RenderingUtilities::DrawLinkedLanes(*Storage, PDI, LaneLoc.LaneHandle, Component->LinkedLanes);
					}

					// Draw lanes from the resulting path (starting from LaneLocation)
					if (Component->bDrawLanePath)
					{
						UE::ZoneGraph::RenderingUtilities::DrawLanePath(*Storage, PDI, Component->LanePath, FColor::Orange, /*thickness=*/12.f);
					}

					Component->ExecuteOnEachCustomTest([&PDI](const UZoneLaneTest& Test) { Test.Draw(PDI); });

					if (Component->bDrawLaneTangentVectors)
					{
						UE::ZoneGraph::RenderingUtilities::DrawLaneDirections(*Storage, PDI, LaneLoc.LaneHandle, FColor(128,16,0));
					}
			
					if (Component->bDrawLaneSmoothing)
					{
						UE::ZoneGraph::RenderingUtilities::DrawLaneSmoothing(*Storage, PDI, LaneLoc.LaneHandle, FColor(64,8,0));
					}
				}
			}
			
			if (Component->bDrawBVTreeQuery)
			{
				UZoneGraphSubsystem* ZoneGraphSubsystem = UWorld::GetSubsystem<UZoneGraphSubsystem>(Component->GetWorld());
				check(ZoneGraphSubsystem);

				const FBox QueryBox(Center - Component->SearchExtent, Center + Component->SearchExtent);

				for (const FRegisteredZoneGraphData& Registered : ZoneGraphSubsystem->GetRegisteredZoneGraphData())
				{
					if (Registered.bInUse && Registered.ZoneGraphData != nullptr)
					{
						const FZoneGraphStorage& Storage = Registered.ZoneGraphData->GetStorage();
						if (Storage.Bounds.Intersect(QueryBox))
						{
							// Draw quantized query box
							FZoneGraphBVNode QueryNode = Storage.ZoneBVTree.CalcNodeBounds(QueryBox);
							DrawWireBox(PDI, Storage.ZoneBVTree.CalcWorldBounds(QueryNode), FColor::Orange, SDPG_World);

							// Draw query results
							Storage.ZoneBVTree.Query(QueryBox, [PDI, Storage](const FZoneGraphBVNode& Node)
							{
								const FBox Bounds = Storage.ZoneBVTree.CalcWorldBounds(Node);
								DrawWireBox(PDI, Bounds, FColor::Red, SDPG_World);
							});
						}
					}
				}
			}
		}

	private:
		const UZoneGraphTestingComponent* Component;
	};

	return new FZoneGraphTestingSceneProxy(*this);
}
#endif // !UE_BUILD_SHIPPING

void UZoneGraphTestingComponent::ExecuteOnEachCustomTest(TFunctionRef<void(UZoneLaneTest&)> ExecFunc)
{
	for (UZoneLaneTest* Test : CustomTests)
	{
		if (Test != nullptr)
		{
			ExecFunc(*Test);
		}
	}
}

void UZoneGraphTestingComponent::ExecuteOnEachCustomTest(TFunctionRef<void(const UZoneLaneTest&)> ExecFunc) const
{
	for (const UZoneLaneTest* Test : CustomTests)
	{
		if (Test != nullptr)
		{
			ExecFunc(*Test);
		}
	}
}


AZoneGraphTestingActor::AZoneGraphTestingActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DebugComp = CreateDefaultSubobject<UZoneGraphTestingComponent>(TEXT("DebugComp"));
	RootComponent = DebugComp;

	SetCanBeDamaged(false);
}

#if WITH_EDITOR
void AZoneGraphTestingActor::PostEditMove(bool bFinished)
{
	if (DebugComp)
	{
		DebugComp->UpdateTests();
	}
}
#endif

void AZoneGraphTestingActor::EnableCustomTests()
{
	if (DebugComp)
	{
		DebugComp->EnableCustomTests();
	}
}

void AZoneGraphTestingActor::DisableCustomTests()
{
	if (DebugComp)
	{
		DebugComp->DisableCustomTests();
	}
}
