// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "UObject/ObjectMacros.h"
#include "Components/PrimitiveComponent.h"
#include "ZoneGraphTypes.h"
#include "Templates/Function.h"
#include "ZoneGraphTestingActor.generated.h"

#define UE_API ZONEGRAPHDEBUG_API

class AZoneGraphData;
class UZoneGraphTestingComponent;
class UZoneGraphSubsystem;
class AZoneGraphTestingActor;

/** Base class to inherit from to be able to perform custom actions on lane detected by the testing actor. */
UCLASS(MinimalAPI, Abstract, EditInlineNew)
class UZoneLaneTest : public UObject
{
	GENERATED_BODY()

public:
	virtual void OnLaneLocationUpdated(const FZoneGraphLaneLocation& PrevLaneLocation, const FZoneGraphLaneLocation& NextLaneLocation) PURE_VIRTUAL(UZoneLaneTest::OnLaneLocationUpdated, );
	virtual void Draw(FPrimitiveDrawInterface* PDI) const {};

	void SetOwner(UZoneGraphTestingComponent* Owner) { OwnerComponent = Owner; OnOwnerSet(); }
	const UZoneGraphTestingComponent* GetOwner() const { return OwnerComponent; }

protected:
	virtual void OnOwnerSet() {}

	UPROPERTY()
	TObjectPtr<UZoneGraphTestingComponent> OwnerComponent;
};

/** Actor for testing ZoneGraph functionality. */
UCLASS(MinimalAPI, ClassGroup = Custom, hidecategories = (Physics, Collision, Rendering, Cooking, Lighting, Navigation, Tags, HLOD, Mobile, AssetUserData, Activation))
class UZoneGraphTestingComponent : public UPrimitiveComponent
{
	GENERATED_BODY()
public:
	UE_API UZoneGraphTestingComponent(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;

	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;

	UE_API void UpdateTests();

#if !UE_BUILD_SHIPPING
	//~ Begin UPrimitiveComponent Interface.
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	//~ End UPrimitiveComponent Interface.
#endif

	/** Returns the graph storage corresponding to the provided lane handle, if any. */
	UE_API const FZoneGraphStorage* GetZoneGraphStorage(const FZoneGraphLaneHandle& LaneHandle) const;

	/** Allow custom tests to be notified when lane location is updated. */
	UFUNCTION(BlueprintCallable, CallInEditor, Category = Test)
	UE_API void EnableCustomTests();

	/** Prevent custom tests to be notified when lane location is updated. Currently active tests will get notified with an invalid location. */
	UFUNCTION(BlueprintCallable, CallInEditor, Category = Test)
	UE_API void DisableCustomTests();

protected:

#if WITH_EDITOR
	UE_API void OnZoneGraphDataBuildDone(const struct FZoneGraphBuildData& BuildData);
#endif
	UE_API void OnZoneGraphDataChanged(const AZoneGraphData* ZoneGraphData);

#if WITH_EDITOR
	FDelegateHandle OnDataChangedHandle;
#endif
	FDelegateHandle OnDataAddedHandle;
	FDelegateHandle OnDataRemovedHandle;

	UPROPERTY(Transient)
	TObjectPtr<UZoneGraphSubsystem> ZoneGraph;

	UPROPERTY(Transient)
	FZoneGraphLaneLocation LaneLocation;

	UPROPERTY(Transient)
	FZoneGraphLaneLocation NextLaneLocation;

	UPROPERTY(Transient)
	FZoneGraphLaneLocation NearestLaneLocation;

	UPROPERTY(EditAnywhere, Category = Test);
	FVector SearchExtent;

	UPROPERTY(EditAnywhere, Category = Test);
	float AdvanceDistance;

	UPROPERTY(EditAnywhere, Category = Test);
	FVector NearestTestOffset;
	
	UPROPERTY(EditAnywhere, Category = Test);
	FZoneGraphTagFilter QueryFilter;

	UPROPERTY(EditAnywhere, Category = Test);
	bool bDrawLinkedLanes = false;

	UPROPERTY(EditAnywhere, Category = Test);
    bool bDrawLaneTangentVectors = false;

	UPROPERTY(EditAnywhere, Category = Test);
	bool bDrawLaneSmoothing = false;

	UPROPERTY(EditAnywhere, Category = Test);
	bool bDrawBVTreeQuery = false;

	/* Experimental */
	UPROPERTY(EditAnywhere, Category = Test);
	bool bDrawLanePath = false;

	UPROPERTY(EditAnywhere, Category = Test)
	TObjectPtr<AZoneGraphTestingActor> OtherActor;

	TArray<FZoneGraphLinkedLane> LinkedLanes;
	FZoneGraphLanePath LanePath;

	UPROPERTY(EditAnywhere, Category = Test, Instanced)
	TArray<TObjectPtr<UZoneLaneTest>> CustomTests;

private:
	void ExecuteOnEachCustomTest(TFunctionRef<void(UZoneLaneTest&)> ExecFunc);
	void ExecuteOnEachCustomTest(TFunctionRef<void(const UZoneLaneTest&)> ExecFunc) const;

	bool bCustomTestsDisabled = false;
};

/** Debug actor to visually test zone graph. */
UCLASS(hidecategories = (Actor, Input, Collision, Rendering, Replication, Partition, HLOD, Cooking))
class AZoneGraphTestingActor : public AActor
{
	GENERATED_BODY()
public:
	AZoneGraphTestingActor(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

#if WITH_EDITOR
	virtual void PostEditMove(bool bFinished) override;
#endif // WITH_EDITOR

	/** Allow custom tests to be notified when lane location is updated. */
	UFUNCTION(BlueprintCallable, CallInEditor, Category = Test)
	void EnableCustomTests();

	/** Prevent custom tests to be notified when lane location is updated. Currently active tests will get notified with an invalid location. */
	UFUNCTION(BlueprintCallable, CallInEditor, Category = Test)
	void DisableCustomTests();

protected:
	UPROPERTY(Category = Default, VisibleAnywhere, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UZoneGraphTestingComponent> DebugComp;
};

#undef UE_API
