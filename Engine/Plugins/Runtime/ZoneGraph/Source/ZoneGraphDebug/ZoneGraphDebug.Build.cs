// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ZoneGraphDebug : ModuleRules
	{
		public ZoneGraphDebug(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

			PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"GameplayTags",
				"GameplayTasks",
				"AIModule",
				"ZoneGraph"
			}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.Add("EditorFramework");
				PrivateDependencyModuleNames.Add("UnrealEd");
			}

			SetupGameplayDebuggerSupport(Target);
		}
	}
}
