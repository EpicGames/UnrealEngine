// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZoneGraphEditorModule.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Editor/UnrealEdEngine.h"
#include "Framework/Commands/Commands.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"
#include "ShowFlags.h"
#include "ToolMenus.h"
#include "UnrealEdGlobals.h"
#include "ZoneGraphDelegates.h"
#include "ZoneGraphEditorStyle.h"
#include "ZoneGraphTagDetails.h"
#include "ZoneGraphTagFilterDetails.h"
#include "ZoneGraphTagInfoDetails.h"
#include "ZoneGraphTagMaskDetails.h"
#include "ZoneGraphTessellationSettingsDetails.h"
#include "ZoneLaneDescDetails.h"
#include "ZoneLaneProfileDetails.h"
#include "ZoneLaneProfileRefDetails.h"
#include "ZoneShapeComponent.h"
#include "ZoneShapeComponentDetails.h"
#include "ZoneShapeComponentVisualizer.h"


#define LOCTEXT_NAMESPACE "ZoneGraphEditor"

IMPLEMENT_MODULE(FZoneGraphEditorModule, ZoneGraphEditor)

class FZoneGraphEditorCommands : public TCommands<FZoneGraphEditorCommands>
{
public:

	FZoneGraphEditorCommands()
		: TCommands<FZoneGraphEditorCommands>(TEXT("ZoneGraphEditorCommands"), NSLOCTEXT("Contexts", "FZoneGraphEditorModule", "Zone Graph Editor Plugin"), FName(), FAppStyle::GetAppStyleSetName())
	{
	}

	// TCommands<> interface
	virtual void RegisterCommands() override
	{
		UI_COMMAND(BuildZoneGraph, "Build ZoneGraph", "Builds ZoneGraph data", EUserInterfaceActionType::Button, FInputChord());
	}

public:
	TSharedPtr<FUICommandInfo> BuildZoneGraph;
};


void FZoneGraphEditorModule::StartupModule()
{
	FZoneGraphEditorCommands::Register();

	FZoneGraphEditorStyle::Initialize();

	PluginCommands = MakeShareable(new FUICommandList);
	PluginCommands->MapAction(
		FZoneGraphEditorCommands::Get().BuildZoneGraph,
		FExecuteAction::CreateRaw(this, &FZoneGraphEditorModule::OnBuildZoneGraph),
		FCanExecuteAction());

	// Register the details customizer
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout(TEXT("ZoneShapeComponent"), FOnGetDetailCustomizationInstance::CreateStatic(&FZoneShapeComponentDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(TEXT("ZoneLaneProfileRef"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FZoneLaneProfileRefDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(TEXT("ZoneLaneProfile"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FZoneLaneProfileDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(TEXT("ZoneLaneDesc"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FZoneLaneDescDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(TEXT("ZoneGraphTag"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FZoneGraphTagDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(TEXT("ZoneGraphTagMask"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FZoneGraphTagMaskDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(TEXT("ZoneGraphTagInfo"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FZoneGraphTagInfoDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(TEXT("ZoneGraphTagFilter"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FZoneGraphTagFilterDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(TEXT("ZoneGraphTessellationSettings"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FZoneGraphTessellationSettingsDetails::MakeInstance));

	RegisterComponentVisualizer(UZoneShapeComponent::StaticClass()->GetFName(), MakeShareable(new FZoneShapeComponentVisualizer));

	UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FZoneGraphEditorModule::RegisterMenus));
}

void FZoneGraphEditorModule::ShutdownModule()
{
	UToolMenus::UnRegisterStartupCallback(this);
	UToolMenus::UnregisterOwner(this);

	PluginCommands.Reset();
	FZoneGraphEditorCommands::Unregister();

	// Unregister the data asset type actions
	if (FModuleManager::Get().IsModuleLoaded("AssetTools"))
	{
	}

	if (FModuleManager::Get().IsModuleLoaded(TEXT("PropertyEditor")))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
		PropertyModule.UnregisterCustomClassLayout(TEXT("ZoneShapeComponent"));
		PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("ZoneLaneProfileRef"));
		PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("ZoneLaneProfile"));
		PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("ZoneLane"));
		PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("ZoneGraphTag"));
		PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("ZoneGraphTagMask"));
		PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("ZoneGraphTagInfo"));
		PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("ZoneGraphTagFilter"));
		PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("ZoneGraphTessellationSettings"));
	}

	if (GEngine)
	{
		// Iterate over all class names we registered for
		for (FName ClassName : RegisteredComponentClassNames)
		{
			GUnrealEd->UnregisterComponentVisualizer(ClassName);
		}
	}
}

void FZoneGraphEditorModule::RegisterComponentVisualizer(FName ComponentClassName, TSharedPtr<FComponentVisualizer> Visualizer)
{
	if (GUnrealEd != NULL)
	{
		GUnrealEd->RegisterComponentVisualizer(ComponentClassName, Visualizer);
	}

	RegisteredComponentClassNames.Add(ComponentClassName);

	if (Visualizer.IsValid())
	{
		Visualizer->OnRegister();
	}
}

void FZoneGraphEditorModule::RegisterMenus()
{
	// Owner will be used for cleanup in call to UToolMenus::UnregisterOwner
	FToolMenuOwnerScoped OwnerScoped(this);

	if (UToolMenu* BuildMenu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu.Build"))
	{
		FToolMenuSection& Section = BuildMenu->FindOrAddSection("LevelEditorNavigation");
		Section.AddMenuEntryWithCommandList(FZoneGraphEditorCommands::Get().BuildZoneGraph, PluginCommands);
	}
}

void FZoneGraphEditorModule::OnBuildZoneGraph()
{
#if WITH_EDITOR
	UE::ZoneGraphDelegates::OnZoneGraphRequestRebuild.Broadcast();
#endif
}


#undef LOCTEXT_NAMESPACE
