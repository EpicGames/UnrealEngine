// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZoneGraphEditorStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/AppStyle.h"
#include "Interfaces/IPluginManager.h"

#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( FMeshEditorStyle::InContent( RelativePath, ".png" ), __VA_ARGS__ )
#define RootToContentDir StyleSet->RootToContentDir

TSharedPtr<FSlateStyleSet> FZoneGraphEditorStyle::StyleSet = nullptr;

FString FZoneGraphEditorStyle::InContent(const FString& RelativePath, const ANSICHAR* Extension)
{
	static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("ZoneGraphEditor"))->GetContentDir() / TEXT("Slate");
	return (ContentDir / RelativePath) + Extension;
}

void FZoneGraphEditorStyle::Initialize()
{
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShared<FSlateStyleSet>(GetStyleSetName());

	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	const FTextBlockStyle& NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");

	StyleSet->Set("ZoneGraph.Tag.Label", FTextBlockStyle(NormalText).SetFont(DEFAULT_FONT("Bold", 7)));

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
}

#undef IMAGE_PLUGIN_BRUSH
#undef RootToContentDir

void FZoneGraphEditorStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}

FName FZoneGraphEditorStyle::GetStyleSetName()
{
	static FName StyleName("ZoneGraphEditorStyle");
	return StyleName;
}
