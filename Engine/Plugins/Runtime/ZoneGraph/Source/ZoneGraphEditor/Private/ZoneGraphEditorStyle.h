// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FZoneGraphEditorStyle
{
public:

	static void Initialize();
	static void Shutdown();
	static TSharedPtr<ISlateStyle> Get() { return StyleSet; }
	static FName GetStyleSetName();

	static const FSlateBrush* GetBrush(FName PropertyName, const ANSICHAR* Specifier = NULL)
	{
		return StyleSet->GetBrush(PropertyName, Specifier);
	}

	static FString InContent(const FString& RelativePath, const ANSICHAR* Extension);

private:
	static TSharedPtr<FSlateStyleSet> StyleSet;
};
