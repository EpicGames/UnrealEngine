// Copyright Epic Games, Inc. All Rights Reserved.
#include "ZoneGraphPropertyUtils.h"
#include "ZoneGraphTypes.h"
#include "ZoneGraphSettings.h"

#define LOCTEXT_NAMESPACE "ZoneGraphEditor"

namespace UE::ZoneGraph::PropertyUtils
{

FLinearColor GetMaskColor(TSharedPtr<IPropertyHandle> MaskProperty)
{
	if (!MaskProperty.IsValid())
	{
		return FLinearColor::Transparent;
	}

	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
	check(ZoneGraphSettings);
	TConstArrayView<FZoneGraphTagInfo> TagInfos = ZoneGraphSettings->GetTagInfos();

	TOptional<FZoneGraphTagMask> TagsOpt = UE::ZoneGraph::PropertyUtils::GetValue<FZoneGraphTagMask>(MaskProperty);
	if (TagsOpt.IsSet())
	{
		FZoneGraphTagMask Tags = TagsOpt.GetValue();
		// Pick the first color
		FLinearColor Color = FLinearColor::Black;
		for (const FZoneGraphTagInfo& Info : TagInfos)
		{
			if (Tags.Contains(Info.Tag))
			{
				if (Info.IsValid())
				{
					Color = FLinearColor(Info.Color);
					break;
				}
			}
		}
		return Color;
	}
	return FLinearColor::Transparent;
}

FText GetMaskDescription(TSharedPtr<IPropertyHandle> MaskProperty)
{
	if (!MaskProperty.IsValid())
	{
		return FText::GetEmpty();
	}

	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
	check(ZoneGraphSettings);
	TConstArrayView<FZoneGraphTagInfo> TagInfos = ZoneGraphSettings->GetTagInfos();

	TOptional<FZoneGraphTagMask> TagsOpt = UE::ZoneGraph::PropertyUtils::GetValue<FZoneGraphTagMask>(MaskProperty);
	if (TagsOpt.IsSet())
	{
		FZoneGraphTagMask Tags = TagsOpt.GetValue();

		TArray<FText> Names;
		for (const FZoneGraphTagInfo& Info : TagInfos)
		{
			if (Tags.Contains(Info.Tag))
			{
				if (Info.IsValid())
				{
					Names.Add(FText::FromName(Info.Name));
				}
			}
		}
		if (Names.Num() == 0)
		{
			return LOCTEXT("EmptyMask", "(Empty)");
		}
		else
		{
			if (Names.Num() > 2)
			{
				Names.SetNum(2);
				Names.Add(FText::FromString(TEXT("...")));
			}
		}
		return FText::Join(FText::FromString(TEXT(", ")), Names);
	}

	return LOCTEXT("MultipleSelected", "(Multiple Selected)");
}

} // UE::ZoneGraph::PropertyUtils

#undef LOCTEXT_NAMESPACE