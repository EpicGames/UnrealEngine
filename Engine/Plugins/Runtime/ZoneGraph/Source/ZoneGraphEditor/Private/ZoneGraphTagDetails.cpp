// Copyright Epic Games, Inc. All Rights Reserved.
#include "ZoneGraphTagDetails.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "IPropertyUtilities.h"
#include "IDetailPropertyRow.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "ZoneGraphSettings.h"
#include "ZoneGraphTypes.h"
#include "ZoneGraphDelegates.h"
#include "Modules/ModuleManager.h"
#include "ISettingsModule.h"

#define LOCTEXT_NAMESPACE "ZoneGraphEditor"

TSharedRef<IPropertyTypeCustomization> FZoneGraphTagDetails::MakeInstance()
{
	return MakeShareable(new FZoneGraphTagDetails);
}

void FZoneGraphTagDetails::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructProperty = StructPropertyHandle;
	PropUtils = StructCustomizationUtils.GetPropertyUtilities().Get();
	BitProperty = StructProperty->GetChildHandle(TEXT("Bit"));

	UE::ZoneGraphDelegates::OnZoneGraphTagsChanged.AddSP(this, &FZoneGraphTagDetails::CacheTagInfos);
	CacheTagInfos();

	HeaderRow
	.NameContent()
	[
		StructProperty->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(200.0f)
	[
		SNew(SComboButton)
		.OnGetMenuContent(this, &FZoneGraphTagDetails::OnGetComboContent)
		.ContentPadding(FMargin(2.0f, 0.0f))
		.ButtonContent()
		[
			SNew(SHorizontalBox)
			// Color
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.MaxWidth(25)
			.VAlign(VAlign_Center)
			[
				SNew(SColorBlock)
				.Color(this, &FZoneGraphTagDetails::GetColor)
				.Size(FVector2D(16.0f, 16.0f))
			]
			// Description
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(FMargin(6.0f, 2.0f))
			[
				SNew(STextBlock)
				.Text(this, &FZoneGraphTagDetails::GetDescription)
				.Font(IDetailLayoutBuilder::GetDetailFontBold())
			]
		]
	];
}

void FZoneGraphTagDetails::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

void FZoneGraphTagDetails::CacheTagInfos()
{
	TagInfos.Reset();
	if (const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>())
	{
		ZoneGraphSettings->GetValidTagInfos(TagInfos);
	}
}


FLinearColor FZoneGraphTagDetails::GetColor() const
{
	uint8 Bit = 0;
	if (BitProperty->GetValue(Bit) == FPropertyAccess::Success)
	{
		// Pick the first color
		FLinearColor Color = FLinearColor::Black;
		for (const FZoneGraphTagInfo& Info : TagInfos)
		{
			if (Info.Tag.Get() == Bit)
			{
				Color = FLinearColor(Info.Color);
				break;
			}
		}
		return Color;
	}
	return FLinearColor::Transparent;
}

FText FZoneGraphTagDetails::GetDescription() const
{
	uint8 Bit = 0;
	FPropertyAccess::Result Result = BitProperty->GetValue(Bit);
	if (Result == FPropertyAccess::Success)
	{
		FText Name = LOCTEXT("NameEmpty", "(Empty)");
		for (const FZoneGraphTagInfo& Info : TagInfos)
		{
			if (Info.Tag.Get() == Bit)
			{
				Name = FText::FromName(Info.Name);
				break;
			}
		}
		return Name;
	}
	else if (Result == FPropertyAccess::MultipleValues)
	{
		return LOCTEXT("MultipleSelected", "(Multiple Selected)");
	}

	return FText::GetEmpty();
}

TSharedRef<SWidget> FZoneGraphTagDetails::OnGetComboContent() const
{
	FMenuBuilder MenuBuilder(true, nullptr);

	FUIAction EditTagsItemAction(FExecuteAction::CreateSP(const_cast<FZoneGraphTagDetails*>(this), &FZoneGraphTagDetails::OnEditTags));
	MenuBuilder.AddMenuEntry(LOCTEXT("EditTags", "Edit Tags..."), TAttribute<FText>(), FSlateIcon(), EditTagsItemAction);
	MenuBuilder.AddMenuSeparator();

	for (const FZoneGraphTagInfo& Info : TagInfos)
	{
		const uint8 Bit = Info.Tag.Get();
		FUIAction BitItemAction(FExecuteAction::CreateSP(const_cast<FZoneGraphTagDetails*>(this), &FZoneGraphTagDetails::OnSetBit, Bit));

		TSharedRef<SWidget> Widgets = SNew(SHorizontalBox)
			// Color
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.MaxWidth(25)
			.VAlign(VAlign_Center)
			[
				SNew(SColorBlock)
				.Color(Info.Color)
				.Size(FVector2D(16.0f, 16.0f))
			]
			// Description
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(FMargin(6.0f, 2.0f))
			[
				SNew(STextBlock)
				.Text(FText::FromName(Info.Name))
			];

		MenuBuilder.AddMenuEntry(BitItemAction, Widgets);
	}

	return MenuBuilder.MakeWidget();
}

void FZoneGraphTagDetails::OnEditTags()
{
	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
	// Goto settings to edit tags
	FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer(ZoneGraphSettings->GetContainerName(), ZoneGraphSettings->GetCategoryName(), ZoneGraphSettings->GetSectionName());
}

void FZoneGraphTagDetails::OnSetBit(uint8 Bit)
{
	BitProperty->SetValue(Bit);
}

#undef LOCTEXT_NAMESPACE