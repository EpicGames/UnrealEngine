// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "UnrealClient.h"
#include "IPropertyTypeCustomization.h"
#include "ZoneGraphTypes.h"

class IPropertyHandle;
class SWidget;

/**
 * Type customization for FZoneGraphTag.
 */
class FZoneGraphTagDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:

	FLinearColor GetColor() const;
	FText GetDescription() const;
	TSharedRef<SWidget> OnGetComboContent() const;

	void OnEditTags();
	void OnSetBit(uint8 Bit);

	void CacheTagInfos();

	TArray<FZoneGraphTagInfo> TagInfos;

	TSharedPtr<IPropertyHandle> BitProperty;

	class IPropertyUtilities* PropUtils;
	TSharedPtr<IPropertyHandle> StructProperty;
};