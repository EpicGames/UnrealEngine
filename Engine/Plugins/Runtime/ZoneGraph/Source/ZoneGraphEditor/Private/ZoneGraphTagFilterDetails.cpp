// Copyright Epic Games, Inc. All Rights Reserved.
#include "ZoneGraphTagFilterDetails.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "ZoneGraphPropertyUtils.h"
#include "ZoneGraphEditorStyle.h"

#define LOCTEXT_NAMESPACE "ZoneGraphEditor"

TSharedRef<IPropertyTypeCustomization> FZoneGraphTagFilterDetails::MakeInstance()
{
	return MakeShareable(new FZoneGraphTagFilterDetails);
}

void FZoneGraphTagFilterDetails::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructProperty = StructPropertyHandle;
	PropUtils = StructCustomizationUtils.GetPropertyUtilities().Get();
	AnyTagsProperty = StructProperty->GetChildHandle(TEXT("AnyTags"));
	AllTagsProperty = StructProperty->GetChildHandle(TEXT("AllTags"));
	NotTagsProperty = StructProperty->GetChildHandle(TEXT("NotTags"));
	AnyTagsMaskProperty = AnyTagsProperty->GetChildHandle(TEXT("Mask"));
	AllTagsMaskProperty = AllTagsProperty->GetChildHandle(TEXT("Mask"));
	NotTagsMaskProperty = NotTagsProperty->GetChildHandle(TEXT("Mask"));

	HeaderRow
	.NameContent()
	[
		StructProperty->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(400.0f)
	[
		SNew(SHorizontalBox)
		
		// Any Tags
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(6.0f, 2.0f))
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ANY", "ANY"))
			.TextStyle(FZoneGraphEditorStyle::Get(), "ZoneGraph.Tag.Label")
			.Visibility(this, &FZoneGraphTagFilterDetails::IsAnyTagsVisible)
		]
		// Color
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.MaxWidth(25)
		.VAlign(VAlign_Center)
		[
			SNew(SColorBlock)
			.Color(this, &FZoneGraphTagFilterDetails::GetAnyTagsColor)
			.Size(FVector2D(16.0f, 16.0f))
			.Visibility(this, &FZoneGraphTagFilterDetails::IsAnyTagsVisible)
		]
		// Description
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(6.0f, 2.0f))
		[
			SNew(STextBlock)
			.Text(this, &FZoneGraphTagFilterDetails::GetAnyTagsDescription)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Visibility(this, &FZoneGraphTagFilterDetails::IsAnyTagsVisible)
		]

		// All Tags
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(6.0f, 2.0f))
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ALL", "ALL"))
			.TextStyle(FZoneGraphEditorStyle::Get(), "ZoneGraph.Tag.Label")
			.Visibility(this, &FZoneGraphTagFilterDetails::IsAllTagsVisible)
		]
		// Color
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.MaxWidth(25)
		.VAlign(VAlign_Center)
		[
			SNew(SColorBlock)
			.Color(this, &FZoneGraphTagFilterDetails::GetAllTagsColor)
			.Size(FVector2D(16.0f, 16.0f))
			.Visibility(this, &FZoneGraphTagFilterDetails::IsAllTagsVisible)
		]
		// Description
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(6.0f, 2.0f))
		[
			SNew(STextBlock)
			.Text(this, &FZoneGraphTagFilterDetails::GetAllTagsDescription)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Visibility(this, &FZoneGraphTagFilterDetails::IsAllTagsVisible)
		]

		// Not Tags
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(12.0f, 2.0f, 6.0f, 2.0f))
		[
			SNew(STextBlock)
			.Text(LOCTEXT("NOT", "NOT"))
			.TextStyle(FZoneGraphEditorStyle::Get(), "ZoneGraph.Tag.Label")
			.Visibility(this, &FZoneGraphTagFilterDetails::IsNotTagsVisible)
		]
		// Color
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.MaxWidth(25)
		.VAlign(VAlign_Center)
		[
			SNew(SColorBlock)
			.Color(this, &FZoneGraphTagFilterDetails::GetNotTagsColor)
			.Size(FVector2D(16.0f, 16.0f))
			.Visibility(this, &FZoneGraphTagFilterDetails::IsNotTagsVisible)
		]
		// Description
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(6.0f, 2.0f))
		[
			SNew(STextBlock)
			.Text(this, &FZoneGraphTagFilterDetails::GetNotTagsDescription)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Visibility(this, &FZoneGraphTagFilterDetails::IsNotTagsVisible)
		]
		
	];
}

void FZoneGraphTagFilterDetails::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 ChildNum = 0;
	if (StructPropertyHandle->GetNumChildren(ChildNum) == FPropertyAccess::Success)
	{
		for (uint32 ChildIndex = 0; ChildIndex < ChildNum; ++ChildIndex)
		{
			TSharedPtr<IPropertyHandle> ChildProperty = StructPropertyHandle->GetChildHandle(ChildIndex);
			if (ChildProperty)
			{
				StructBuilder.AddProperty(ChildProperty.ToSharedRef());
			}
		}
	}
}

FLinearColor FZoneGraphTagFilterDetails::GetAnyTagsColor() const
{
	return UE::ZoneGraph::PropertyUtils::GetMaskColor(AnyTagsProperty);
}

FText FZoneGraphTagFilterDetails::GetAnyTagsDescription() const
{
	return UE::ZoneGraph::PropertyUtils::GetMaskDescription(AnyTagsProperty);
}

EVisibility FZoneGraphTagFilterDetails::IsAnyTagsVisible() const
{
	uint32 Mask = 0;
	FPropertyAccess::Result Result = AnyTagsMaskProperty->GetValue(Mask);
	return Mask != 0 ? EVisibility::Visible : EVisibility::Collapsed;
}


FLinearColor FZoneGraphTagFilterDetails::GetAllTagsColor() const
{
	return UE::ZoneGraph::PropertyUtils::GetMaskColor(AllTagsProperty);
}

FText FZoneGraphTagFilterDetails::GetAllTagsDescription() const
{
	return UE::ZoneGraph::PropertyUtils::GetMaskDescription(AllTagsProperty);
}

EVisibility FZoneGraphTagFilterDetails::IsAllTagsVisible() const
{
	uint32 Mask = 0;
	FPropertyAccess::Result Result = AllTagsMaskProperty->GetValue(Mask);
	return Mask != 0 ? EVisibility::Visible : EVisibility::Collapsed;
}


FLinearColor FZoneGraphTagFilterDetails::GetNotTagsColor() const
{
	return UE::ZoneGraph::PropertyUtils::GetMaskColor(NotTagsProperty);
}

FText FZoneGraphTagFilterDetails::GetNotTagsDescription() const
{
	return UE::ZoneGraph::PropertyUtils::GetMaskDescription(NotTagsProperty);
}

EVisibility FZoneGraphTagFilterDetails::IsNotTagsVisible() const
{
	uint32 Mask = 0;
	FPropertyAccess::Result Result = NotTagsMaskProperty->GetValue(Mask);
	return Mask != 0 ? EVisibility::Visible : EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE