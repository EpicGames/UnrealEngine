// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "UnrealClient.h"
#include "IPropertyTypeCustomization.h"
#include "ZoneGraphTypes.h"

class IPropertyHandle;

/**
 * Type customization for FZoneGraphTagFilter.
 */
class FZoneGraphTagFilterDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:

	FLinearColor GetAnyTagsColor() const;
	FText GetAnyTagsDescription() const;
	EVisibility IsAnyTagsVisible() const;

	FLinearColor GetAllTagsColor() const;
	FText GetAllTagsDescription() const;
	EVisibility IsAllTagsVisible() const;

	FLinearColor GetNotTagsColor() const;
	FText GetNotTagsDescription() const;
	EVisibility IsNotTagsVisible() const;

	TSharedPtr<IPropertyHandle> AnyTagsProperty;
	TSharedPtr<IPropertyHandle> AllTagsProperty;
	TSharedPtr<IPropertyHandle> NotTagsProperty;
	TSharedPtr<IPropertyHandle> AnyTagsMaskProperty;
	TSharedPtr<IPropertyHandle> AllTagsMaskProperty;
	TSharedPtr<IPropertyHandle> NotTagsMaskProperty;

	class IPropertyUtilities* PropUtils;
	TSharedPtr<IPropertyHandle> StructProperty;
};