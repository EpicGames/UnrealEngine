// Copyright Epic Games, Inc. All Rights Reserved.
#include "ZoneGraphTagInfoDetails.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "IPropertyUtilities.h"
#include "IDetailPropertyRow.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "ZoneGraphSettings.h"
#include "ZoneGraphTypes.h"
#include "ZoneGraphPropertyUtils.h"
#include "Modules/ModuleManager.h"
#include "Editor.h"
#include "ISettingsModule.h"

#define LOCTEXT_NAMESPACE "ZoneGraphEditor"


TSharedRef<IPropertyTypeCustomization> FZoneGraphTagInfoDetails::MakeInstance()
{
	return MakeShareable(new FZoneGraphTagInfoDetails);
}

void FZoneGraphTagInfoDetails::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructProperty = StructPropertyHandle;
	PropUtils = StructCustomizationUtils.GetPropertyUtilities().Get();

	NameProperty = StructProperty->GetChildHandle(TEXT("Name"));
	ColorProperty = StructProperty->GetChildHandle(TEXT("Color"));
	TagProperty = StructProperty->GetChildHandle(TEXT("Tag"));

	HeaderRow
	.WholeRowContent()
	[
		SNew(SHorizontalBox)
		// Tag Number
		+ SHorizontalBox::Slot()
		.FillWidth(0.1f)
		.MaxWidth(30.0f)
		.VAlign(VAlign_Center)
		.Padding(FMargin(6.0f, 2.0f))
		[
			SNew(STextBlock)
			.Text(this, &FZoneGraphTagInfoDetails::GetTagDescription)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.MaxWidth(25)
		.VAlign(VAlign_Center)
		[
			SAssignNew(ColorWidget, SColorBlock)
			.Color(this, &FZoneGraphTagInfoDetails::GetColor)
			.Size(FVector2D(16.0f, 16.0f))
			.OnMouseButtonDown(this, &FZoneGraphTagInfoDetails::OnColorPressed)
		]
		// Description
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		.Padding(FMargin(6.0f, 2.0f))
		[
			NameProperty->CreatePropertyValueWidget()
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(12.0f, 0.0f))
		.HAlign(HAlign_Right)
		[
			StructPropertyHandle->CreateDefaultPropertyButtonWidgets()
		]
	];
}

FReply FZoneGraphTagInfoDetails::OnColorPressed(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}

	TOptional<FColor> Color = UE::ZoneGraph::PropertyUtils::GetValue<FColor>(ColorProperty);

	FColorPickerArgs PickerArgs;
	PickerArgs.bOnlyRefreshOnMouseUp = true;
	PickerArgs.ParentWidget = ColorWidget;
	PickerArgs.bUseAlpha = false;
	PickerArgs.bOnlyRefreshOnOk = true;
	PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));
	PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &FZoneGraphTagInfoDetails::SetColor);
	PickerArgs.OnColorPickerCancelled = FOnColorPickerCancelled::CreateSP(this, &FZoneGraphTagInfoDetails::OnColorPickerCancelled);
	PickerArgs.InitialColor = Color.Get(FColor(128, 128, 128));
	OpenColorPicker(PickerArgs);

	return FReply::Handled();
}

void FZoneGraphTagInfoDetails::SetColor(FLinearColor NewColor)
{
	UE::ZoneGraph::PropertyUtils::SetValue<FColor>(ColorProperty, NewColor.ToFColor(true /*bSRGB*/));
}

void FZoneGraphTagInfoDetails::OnColorPickerCancelled(FLinearColor OriginalColor)
{
	UE::ZoneGraph::PropertyUtils::SetValue<FColor>(ColorProperty, OriginalColor.ToFColor(true /*bSRGB*/));
}

void FZoneGraphTagInfoDetails::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

FText FZoneGraphTagInfoDetails::GetTagDescription() const
{
	FZoneGraphTag Tag = UE::ZoneGraph::PropertyUtils::GetValue<FZoneGraphTag>(TagProperty).Get(FZoneGraphTag());
	if (Tag.IsValid())
	{
		return FText::AsNumber(Tag.Get());
	}
	return FText::GetEmpty();
}

FLinearColor FZoneGraphTagInfoDetails::GetColor() const
{
	TOptional<FColor> Color = UE::ZoneGraph::PropertyUtils::GetValue<FColor>(ColorProperty);
	return Color.Get(FColor(128, 128, 128));
}

#undef LOCTEXT_NAMESPACE