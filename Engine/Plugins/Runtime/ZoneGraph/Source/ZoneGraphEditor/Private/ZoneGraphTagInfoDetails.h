// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "UnrealClient.h"
#include "IPropertyTypeCustomization.h"
#include "ZoneGraphTypes.h"

class FReply;
class IPropertyHandle;
class SColorBlock;
struct FGeometry;
struct FPointerEvent;

/**
 * Type customization for FZoneGraphTagInfo.
 */
class FZoneGraphTagInfoDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:

	FLinearColor GetColor() const;
	FText GetTagDescription() const;

	FReply OnColorPressed(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	void SetColor(FLinearColor NewColor);
	void OnColorPickerCancelled(FLinearColor OriginalColor);

	TSharedPtr<IPropertyHandle> TagProperty;
	TSharedPtr<IPropertyHandle> NameProperty;
	TSharedPtr<IPropertyHandle> ColorProperty;
	TSharedPtr<SColorBlock> ColorWidget;

	class IPropertyUtilities* PropUtils;
	TSharedPtr<IPropertyHandle> StructProperty;

	TArray<FZoneGraphTagInfo> Infos;
};