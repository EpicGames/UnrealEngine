// Copyright Epic Games, Inc. All Rights Reserved.
#include "ZoneGraphTagMaskDetails.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "IPropertyUtilities.h"
#include "IDetailPropertyRow.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "ZoneGraphSettings.h"
#include "ZoneGraphTypes.h"
#include "ZoneGraphDelegates.h"
#include "ZoneGraphPropertyUtils.h"
#include "Modules/ModuleManager.h"
#include "ISettingsModule.h"

#define LOCTEXT_NAMESPACE "ZoneGraphEditor"

TSharedRef<IPropertyTypeCustomization> FZoneGraphTagMaskDetails::MakeInstance()
{
	return MakeShareable(new FZoneGraphTagMaskDetails);
}

void FZoneGraphTagMaskDetails::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructProperty = StructPropertyHandle;
	PropUtils = StructCustomizationUtils.GetPropertyUtilities().Get();
	MaskProperty = StructProperty->GetChildHandle(TEXT("Mask"));
	check(MaskProperty.IsValid());

	HeaderRow
	.NameContent()
	[
		StructProperty->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(200.0f)
	[
		SNew(SComboButton)
		.OnGetMenuContent(this, &FZoneGraphTagMaskDetails::OnGetComboContent)
		.MenuPlacement(MenuPlacement_BelowAnchor) // To prevent popup flickering when content changes.
		.ContentPadding(FMargin(2.0f, 0.0f))
		.ButtonContent()
		[
			SNew(SHorizontalBox)
			// Color
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.MaxWidth(25)
			.VAlign(VAlign_Center)
			[
				SNew(SColorBlock)
				.Color(this, &FZoneGraphTagMaskDetails::GetColor)
				.Size(FVector2D(16.0f, 16.0f))
			]
			// Description
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(FMargin(6.0f, 2.0f))
			[
				SNew(STextBlock)
				.Text(this, &FZoneGraphTagMaskDetails::GetDescription)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		]
	];
}

void FZoneGraphTagMaskDetails::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}


FLinearColor FZoneGraphTagMaskDetails::GetColor() const
{
	return UE::ZoneGraph::PropertyUtils::GetMaskColor(StructProperty);
}

FText FZoneGraphTagMaskDetails::GetDescription() const
{
	return UE::ZoneGraph::PropertyUtils::GetMaskDescription(StructProperty);
}

TSharedRef<SWidget> FZoneGraphTagMaskDetails::OnGetComboContent() const
{
	FMenuBuilder MenuBuilder(false, nullptr);

	TConstArrayView<FZoneGraphTagInfo> TagInfos;
	if (const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>())
	{
		TagInfos = ZoneGraphSettings->GetTagInfos();
	}

	FUIAction EditTagsItemAction(FExecuteAction::CreateSP(const_cast<FZoneGraphTagMaskDetails*>(this), &FZoneGraphTagMaskDetails::OnEditTags));
	MenuBuilder.AddMenuEntry(LOCTEXT("EditTags", "Edit Tags..."), TAttribute<FText>(), FSlateIcon(), EditTagsItemAction);
	MenuBuilder.AddMenuSeparator();

	FUIAction SetAllItemAction(FExecuteAction::CreateSP(const_cast<FZoneGraphTagMaskDetails*>(this), &FZoneGraphTagMaskDetails::OnSetMask, FZoneGraphTagMask::All));
	MenuBuilder.AddMenuEntry(LOCTEXT("SetAll", "All"), TAttribute<FText>(), FSlateIcon(), SetAllItemAction);
	FUIAction SetNoneItemAction(FExecuteAction::CreateSP(const_cast<FZoneGraphTagMaskDetails*>(this), &FZoneGraphTagMaskDetails::OnSetMask, FZoneGraphTagMask::None));
	MenuBuilder.AddMenuEntry(LOCTEXT("SetNone", "None"), TAttribute<FText>(), FSlateIcon(), SetNoneItemAction);
	MenuBuilder.AddMenuSeparator();

	for (const FZoneGraphTagInfo& Info : TagInfos)
	{
		const uint32 BitMask = uint32(1) << Info.Tag.Get();
		if (Info.IsValid())
		{
			FUIAction BitItemAction
			(
				FExecuteAction::CreateSP(const_cast<FZoneGraphTagMaskDetails*>(this), &FZoneGraphTagMaskDetails::OnToggleBit, BitMask),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(const_cast<FZoneGraphTagMaskDetails*>(this), &FZoneGraphTagMaskDetails::OnIsBitSet, BitMask)
			);

			FText BitName = FText::FromName(Info.Name);

			TSharedRef<SWidget> Widgets = SNew(SHorizontalBox)
				// Color
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.MaxWidth(25)
				.VAlign(VAlign_Center)
				[
					SNew(SColorBlock)
					.Color(Info.Color)
					.Size(FVector2D(16.0f, 16.0f))
				]
				// Description
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(FMargin(6.0f, 2.0f))
				[
					SNew(STextBlock)
					.Text(BitName)
				];

			MenuBuilder.AddMenuEntry(BitItemAction, Widgets, FName(), TAttribute<FText>(), EUserInterfaceActionType::Check);
		}

	}

	return MenuBuilder.MakeWidget();
}

void FZoneGraphTagMaskDetails::OnEditTags()
{
	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
	// Goto settings to edit tags
	FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer(ZoneGraphSettings->GetContainerName(), ZoneGraphSettings->GetCategoryName(), ZoneGraphSettings->GetSectionName());
}

void FZoneGraphTagMaskDetails::OnToggleBit(uint32 BitMask)
{
	uint32 Mask = 0;
	if (MaskProperty->GetValue(Mask) == FPropertyAccess::Success)
	{
		Mask ^= BitMask;
		MaskProperty->SetValue(Mask);
	}
}

void FZoneGraphTagMaskDetails::OnSetMask(FZoneGraphTagMask Mask)
{
	MaskProperty->SetValue(Mask.GetValue());
}

bool FZoneGraphTagMaskDetails::OnIsBitSet(uint32 BitMask) const
{
	uint32 Mask = 0;
	if (MaskProperty->GetValue(Mask) == FPropertyAccess::Success)
	{
		return (Mask & BitMask) != 0;
	}
	return false;
}


#undef LOCTEXT_NAMESPACE