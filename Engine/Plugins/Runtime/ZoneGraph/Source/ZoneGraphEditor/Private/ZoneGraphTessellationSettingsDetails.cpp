// Copyright Epic Games, Inc. All Rights Reserved.
#include "ZoneGraphTessellationSettingsDetails.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "ZoneGraphTypes.h"
#include "ZoneGraphPropertyUtils.h"
#include "ZoneGraphEditorStyle.h"

#define LOCTEXT_NAMESPACE "ZoneGraphEditor"

TSharedRef<IPropertyTypeCustomization> FZoneGraphTessellationSettingsDetails::MakeInstance()
{
	return MakeShareable(new FZoneGraphTessellationSettingsDetails);
}

void FZoneGraphTessellationSettingsDetails::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructProperty = StructPropertyHandle;
	PropUtils = StructCustomizationUtils.GetPropertyUtilities().Get();

	TSharedPtr<class IPropertyHandle> LaneFilterProperty = StructProperty->GetChildHandle(TEXT("LaneFilter"));
	check(LaneFilterProperty.IsValid());
	AnyTagsProperty = LaneFilterProperty->GetChildHandle(TEXT("AnyTags"));
	AllTagsProperty = LaneFilterProperty->GetChildHandle(TEXT("AllTags"));
	NotTagsProperty = LaneFilterProperty->GetChildHandle(TEXT("NotTags"));
	AnyTagsMaskProperty = AnyTagsProperty->GetChildHandle(TEXT("Mask"));
	AllTagsMaskProperty = AllTagsProperty->GetChildHandle(TEXT("Mask"));
	NotTagsMaskProperty = NotTagsProperty->GetChildHandle(TEXT("Mask"));
	ToleranceProperty = StructProperty->GetChildHandle(TEXT("TessellationTolerance"));

	HeaderRow
	.WholeRowContent()
	[
		SNew(SHorizontalBox)
		
		// Any Tags
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(6.0f, 2.0f))
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ANY", "ANY"))
			.TextStyle(FZoneGraphEditorStyle::Get(), "ZoneGraph.Tag.Label")
			.Visibility(this, &FZoneGraphTessellationSettingsDetails::IsAnyTagsVisible)
		]
		// Color
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.MaxWidth(25)
		.VAlign(VAlign_Center)
		[
			SNew(SColorBlock)
			.Color(this, &FZoneGraphTessellationSettingsDetails::GetAnyTagsColor)
			.Size(FVector2D(16.0f, 16.0f))
			.Visibility(this, &FZoneGraphTessellationSettingsDetails::IsAnyTagsVisible)
		]
		// Description
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(6.0f, 2.0f))
		[
			SNew(STextBlock)
			.Text(this, &FZoneGraphTessellationSettingsDetails::GetAnyTagsDescription)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Visibility(this, &FZoneGraphTessellationSettingsDetails::IsAnyTagsVisible)
		]

		// All Tags
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(6.0f, 2.0f))
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ALL", "ALL"))
			.TextStyle(FZoneGraphEditorStyle::Get(), "ZoneGraph.Tag.Label")
			.Visibility(this, &FZoneGraphTessellationSettingsDetails::IsAllTagsVisible)
		]
		// Color
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.MaxWidth(25)
		.VAlign(VAlign_Center)
		[
			SNew(SColorBlock)
			.Color(this, &FZoneGraphTessellationSettingsDetails::GetAllTagsColor)
			.Size(FVector2D(16.0f, 16.0f))
			.Visibility(this, &FZoneGraphTessellationSettingsDetails::IsAllTagsVisible)
		]
		// Description
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(6.0f, 2.0f))
		[
			SNew(STextBlock)
			.Text(this, &FZoneGraphTessellationSettingsDetails::GetAllTagsDescription)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Visibility(this, &FZoneGraphTessellationSettingsDetails::IsAllTagsVisible)
		]

		// Not Tags
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(12.0f, 2.0f, 6.0f, 2.0f))
		[
			SNew(STextBlock)
			.Text(LOCTEXT("NOT", "NOT"))
			.TextStyle(FZoneGraphEditorStyle::Get(), "ZoneGraph.Tag.Label")
			.Visibility(this, &FZoneGraphTessellationSettingsDetails::IsNotTagsVisible)
		]
		// Color
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.MaxWidth(25)
		.VAlign(VAlign_Center)
		[
			SNew(SColorBlock)
			.Color(this, &FZoneGraphTessellationSettingsDetails::GetNotTagsColor)
			.Size(FVector2D(16.0f, 16.0f))
			.Visibility(this, &FZoneGraphTessellationSettingsDetails::IsNotTagsVisible)
		]
		// Description
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(6.0f, 2.0f))
		[
			SNew(STextBlock)
			.Text(this, &FZoneGraphTessellationSettingsDetails::GetNotTagsDescription)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Visibility(this, &FZoneGraphTessellationSettingsDetails::IsNotTagsVisible)
		]

		// Tolerance
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(FMargin(6.0f, 2.0f, 20.0f, 2.0f))
		[
			SNew(STextBlock)
			.Text(this, &FZoneGraphTessellationSettingsDetails::GetToleranceDescription)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(12.0f, 0.0f))
		.HAlign(HAlign_Right)
		[
			StructPropertyHandle->CreateDefaultPropertyButtonWidgets()
		]
	];
}

void FZoneGraphTessellationSettingsDetails::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 ChildNum = 0;
	if (StructPropertyHandle->GetNumChildren(ChildNum) == FPropertyAccess::Success)
	{
		for (uint32 ChildIndex = 0; ChildIndex < ChildNum; ++ChildIndex)
		{
			TSharedPtr<IPropertyHandle> ChildProperty = StructPropertyHandle->GetChildHandle(ChildIndex);
			if (ChildProperty)
			{
				StructBuilder.AddProperty(ChildProperty.ToSharedRef());
			}
		}
	}
}

FLinearColor FZoneGraphTessellationSettingsDetails::GetAnyTagsColor() const
{
	return UE::ZoneGraph::PropertyUtils::GetMaskColor(AnyTagsProperty);
}

FText FZoneGraphTessellationSettingsDetails::GetAnyTagsDescription() const
{
	return UE::ZoneGraph::PropertyUtils::GetMaskDescription(AnyTagsProperty);
}

EVisibility FZoneGraphTessellationSettingsDetails::IsAnyTagsVisible() const
{
	uint32 Mask = 0;
	FPropertyAccess::Result Result = AnyTagsMaskProperty->GetValue(Mask);
	return Mask != 0 ? EVisibility::Visible : EVisibility::Collapsed;
}


FLinearColor FZoneGraphTessellationSettingsDetails::GetAllTagsColor() const
{
	return UE::ZoneGraph::PropertyUtils::GetMaskColor(AllTagsProperty);
}

FText FZoneGraphTessellationSettingsDetails::GetAllTagsDescription() const
{
	return UE::ZoneGraph::PropertyUtils::GetMaskDescription(AllTagsProperty);
}

EVisibility FZoneGraphTessellationSettingsDetails::IsAllTagsVisible() const
{
	uint32 Mask = 0;
	FPropertyAccess::Result Result = AllTagsMaskProperty->GetValue(Mask);
	return Mask != 0 ? EVisibility::Visible : EVisibility::Collapsed;
}


FLinearColor FZoneGraphTessellationSettingsDetails::GetNotTagsColor() const
{
	return UE::ZoneGraph::PropertyUtils::GetMaskColor(NotTagsProperty);
}

FText FZoneGraphTessellationSettingsDetails::GetNotTagsDescription() const
{
	return UE::ZoneGraph::PropertyUtils::GetMaskDescription(NotTagsProperty);
}

EVisibility FZoneGraphTessellationSettingsDetails::IsNotTagsVisible() const
{
	uint32 Mask = 0;
	FPropertyAccess::Result Result = NotTagsMaskProperty->GetValue(Mask);
	return Mask != 0 ? EVisibility::Visible : EVisibility::Collapsed;
}

FText FZoneGraphTessellationSettingsDetails::GetToleranceDescription() const
{
	float Tolerance = 0.0f;
	ToleranceProperty->GetValue(Tolerance);
	return FText::AsNumber(Tolerance);
}


#undef LOCTEXT_NAMESPACE