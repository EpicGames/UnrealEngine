// Copyright Epic Games, Inc. All Rights Reserved.
#include "ZoneLaneDescDetails.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "IPropertyUtilities.h"
#include "IDetailPropertyRow.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "ZoneGraphSettings.h"
#include "ZoneGraphTypes.h"
#include "ZoneGraphPropertyUtils.h"
#include "Modules/ModuleManager.h"
#include "ISettingsModule.h"

#define LOCTEXT_NAMESPACE "ZoneGraphEditor"

TSharedRef<IPropertyTypeCustomization> FZoneLaneDescDetails::MakeInstance()
{
	return MakeShareable(new FZoneLaneDescDetails);
}

void FZoneLaneDescDetails::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructProperty = StructPropertyHandle;
	PropUtils = StructCustomizationUtils.GetPropertyUtilities().Get();

	WidthProperty = StructProperty->GetChildHandle(TEXT("Width"));
	DirectionProperty = StructProperty->GetChildHandle(TEXT("Direction"));
	TagsProperty = StructProperty->GetChildHandle(TEXT("Tags"));

	HeaderRow
	.WholeRowContent()
	[
		SNew(SHorizontalBox)
		// Tags Color
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.MaxWidth(25)
		.VAlign(VAlign_Center)
		[
			SNew(SColorBlock)
			.Color(this, &FZoneLaneDescDetails::GetTagColor)
			.Size(FVector2D(16.0f, 16.0f))
		]
		// Tags Description
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		.Padding(FMargin(6.0f, 2.0f))
		[
			SNew(STextBlock)
			.Text(this, &FZoneLaneDescDetails::GetTagDescription)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		// Description
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		.Padding(FMargin(12.0f, 2.0f, 6.0f, 2.0f))
		[
			SNew(STextBlock)
			.Text(this, &FZoneLaneDescDetails::GetDescription)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(12.0f, 0.0f))
		.HAlign(HAlign_Right)
		[
			StructPropertyHandle->CreateDefaultPropertyButtonWidgets()
		]
	];
}

void FZoneLaneDescDetails::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 ChildNum = 0;
	if (StructPropertyHandle->GetNumChildren(ChildNum) == FPropertyAccess::Success)
	{
		for (uint32 ChildIndex = 0; ChildIndex < ChildNum; ++ChildIndex)
		{
			TSharedPtr<IPropertyHandle> ChildProperty = StructPropertyHandle->GetChildHandle(ChildIndex);
			if (ChildProperty)
			{
				StructBuilder.AddProperty(ChildProperty.ToSharedRef());
			}
		}
	}
}

FLinearColor FZoneLaneDescDetails::GetTagColor() const
{
	return UE::ZoneGraph::PropertyUtils::GetMaskColor(TagsProperty);
}

FText FZoneLaneDescDetails::GetTagDescription() const
{
	return UE::ZoneGraph::PropertyUtils::GetMaskDescription(TagsProperty);
}

FText FZoneLaneDescDetails::GetDescription() const
{
	FText Desc;
	FText Delimiter = FText::FromString(", ");

	float Width = 0.0f;
	if (WidthProperty && WidthProperty->GetValue(Width) == FPropertyAccess::Success)
	{
		FText WidthDesc = FText::AsNumber(Width);
		Desc = Desc.IsEmpty() ? WidthDesc : FText::Join(Delimiter, Desc, WidthDesc);
	}

	uint8 Direction = 0;
	if (DirectionProperty && DirectionProperty->GetValue(Direction) == FPropertyAccess::Success)
	{
		UEnum* Enum = StaticEnum<EZoneLaneDirection>();
		if (Enum)
		{
			FText DirectionDesc = Enum->GetDisplayNameTextByValue(Direction);
			Desc = Desc.IsEmpty() ? DirectionDesc : FText::Join(Delimiter, Desc, DirectionDesc);
		}
	}

	return Desc;
}


#undef LOCTEXT_NAMESPACE