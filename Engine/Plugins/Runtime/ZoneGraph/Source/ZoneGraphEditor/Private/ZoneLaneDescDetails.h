// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "UnrealClient.h"
#include "IPropertyTypeCustomization.h"

class IPropertyHandle;

/**
 * Type customization for FZoneLaneDesc.
 */
class FZoneLaneDescDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:

	FLinearColor GetTagColor() const;
	FText GetTagDescription() const;
	FText GetDescription() const;

	TSharedPtr<IPropertyHandle> WidthProperty;
	TSharedPtr<IPropertyHandle> DirectionProperty;
	TSharedPtr<IPropertyHandle> TagsProperty;

	class IPropertyUtilities* PropUtils;
	TSharedPtr<IPropertyHandle> StructProperty;
};