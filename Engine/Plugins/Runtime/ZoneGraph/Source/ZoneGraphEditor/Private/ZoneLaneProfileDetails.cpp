// Copyright Epic Games, Inc. All Rights Reserved.
#include "ZoneLaneProfileDetails.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "IPropertyUtilities.h"
#include "IDetailPropertyRow.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Input/SComboButton.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "ZoneGraphSettings.h"
#include "ZoneGraphTypes.h"
#include "Modules/ModuleManager.h"
#include "ISettingsModule.h"

#define LOCTEXT_NAMESPACE "ZoneGraphEditor"

TSharedRef<IPropertyTypeCustomization> FZoneLaneProfileDetails::MakeInstance()
{
	return MakeShareable(new FZoneLaneProfileDetails);
}

void FZoneLaneProfileDetails::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructProperty = StructPropertyHandle;
	PropUtils = StructCustomizationUtils.GetPropertyUtilities().Get();

	NameProperty = StructProperty->GetChildHandle(TEXT("Name"));
	LanesProperty = StructProperty->GetChildHandle(TEXT("Lanes"));

	HeaderRow
		.WholeRowContent()
		[
			SNew(SHorizontalBox)
			// Description
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.Padding(FMargin(6.0f, 2.0f))
			[
				SNew(STextBlock)
				.Text(this, &FZoneLaneProfileDetails::GetDescription)
				.Font(IDetailLayoutBuilder::GetDetailFontBold())
			]
			+ SHorizontalBox::Slot()
			.Padding(FMargin(12.0f, 0.0f))
			.HAlign(HAlign_Right)
			[
				StructPropertyHandle->CreateDefaultPropertyButtonWidgets()
			]
		];
}

void FZoneLaneProfileDetails::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 ChildNum = 0;
	if (StructPropertyHandle->GetNumChildren(ChildNum) == FPropertyAccess::Success)
	{
		for (uint32 ChildIndex = 0; ChildIndex < ChildNum; ++ChildIndex)
		{
			TSharedPtr<IPropertyHandle> ChildProperty = StructPropertyHandle->GetChildHandle(ChildIndex);
			if (ChildProperty)
			{
				StructBuilder.AddProperty(ChildProperty.ToSharedRef());
			}
		}
	}
}

FText FZoneLaneProfileDetails::GetDescription() const
{
	FName Name;
	if (NameProperty && NameProperty->GetValue(Name) == FPropertyAccess::Success)
	{
		return FText::FromName(Name);
	}
	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE