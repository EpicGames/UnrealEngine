// Copyright Epic Games, Inc. All Rights Reserved.
#include "ZoneLaneProfileRefDetails.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "IPropertyUtilities.h"
#include "IDetailPropertyRow.h"
#include "IDetailChildrenBuilder.h"
#include "Widgets/Input/SComboButton.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ScopedTransaction.h"
#include "ZoneGraphSettings.h"
#include "ZoneGraphTypes.h"
#include "Modules/ModuleManager.h"
#include "ISettingsModule.h"
#include "Editor.h"
#include "ZoneGraphPropertyUtils.h"

#define LOCTEXT_NAMESPACE "ZoneGraphEditor"

TSharedRef<IPropertyTypeCustomization> FZoneLaneProfileRefDetails::MakeInstance()
{
	return MakeShareable(new FZoneLaneProfileRefDetails);
}

void FZoneLaneProfileRefDetails::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	StructProperty = StructPropertyHandle;
	PropUtils = StructCustomizationUtils.GetPropertyUtilities().Get();
	
	NameProperty = StructProperty->GetChildHandle(TEXT("Name"));
	IDProperty = StructProperty->GetChildHandle(TEXT("ID"));

	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &FZoneLaneProfileRefDetails::OnGetProfileContent)
			.ContentPadding(FMargin(6.0f, 2.0f))
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(this, &FZoneLaneProfileRefDetails::GetCurrentProfileDesc)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];
}

void FZoneLaneProfileRefDetails::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

void FZoneLaneProfileRefDetails::OnProfileComboChange(int32 Idx)
{
	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
	if (Idx == -1)
	{
		// Goto settings to create new Profile
		FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer(ZoneGraphSettings->GetContainerName(), ZoneGraphSettings->GetCategoryName(), ZoneGraphSettings->GetSectionName());
		return;
	}

	const TArray<FZoneLaneProfile>& LaneProfiles = ZoneGraphSettings->GetLaneProfiles();

	if (Idx >= 0 && Idx < LaneProfiles.Num())
	{
		const FZoneLaneProfile& LaneProfile = LaneProfiles[Idx];

		FScopedTransaction Transaction(FText::Format(LOCTEXT("SetPropertyValue", "Set {0}"), StructProperty->GetPropertyDisplayName()));

		if (NameProperty)
		{
			NameProperty->SetValue(LaneProfile.Name, EPropertyValueSetFlags::NotTransactable);
		}

		if (IDProperty)
		{
			UE::ZoneGraph::PropertyUtils::SetValue<FGuid>(IDProperty, LaneProfile.ID, EPropertyValueSetFlags::NotTransactable);
		}

		if (PropUtils)
		{
			PropUtils->ForceRefresh();
		}
	}
}

TSharedRef<SWidget> FZoneLaneProfileRefDetails::OnGetProfileContent() const
{
	FMenuBuilder MenuBuilder(true, NULL);
	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();

	FUIAction NewItemAction(FExecuteAction::CreateSP(const_cast<FZoneLaneProfileRefDetails*>(this), &FZoneLaneProfileRefDetails::OnProfileComboChange, -1));
	MenuBuilder.AddMenuEntry(LOCTEXT("CreateOrEditLaneProfile", "Create or Edit Lane Profile..."), TAttribute<FText>(), FSlateIcon(), NewItemAction);
	MenuBuilder.AddMenuSeparator();

	const TArray<FZoneLaneProfile>& LaneProfiles = ZoneGraphSettings->GetLaneProfiles();
	for (int32 Index = 0; Index < LaneProfiles.Num(); Index++)
	{
		const FZoneLaneProfile& LaneProfile = LaneProfiles[Index];
		FUIAction ItemAction(FExecuteAction::CreateSP(const_cast<FZoneLaneProfileRefDetails*>(this), &FZoneLaneProfileRefDetails::OnProfileComboChange, (int)Index));
		MenuBuilder.AddMenuEntry(FText::FromName(LaneProfile.Name), TAttribute<FText>(), FSlateIcon(), ItemAction);
	}
	return MenuBuilder.MakeWidget();
}

FText FZoneLaneProfileRefDetails::GetCurrentProfileDesc() const
{
	TOptional<FGuid> ProfileIDOpt = UE::ZoneGraph::PropertyUtils::GetValue<FGuid>(IDProperty);
	if (ProfileIDOpt.IsSet())
	{
		const FGuid ProfileID = ProfileIDOpt.GetValue();
		if (ProfileID.IsValid())
		{
			const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
			const FZoneLaneProfile* LaneProfile = ZoneGraphSettings->GetLaneProfileByID(ProfileID);
			if (LaneProfile)
			{
				return FText::FromName(LaneProfile->Name);
			}
			else
			{
				FName OldProfileName;
				if (NameProperty && NameProperty->GetValue(OldProfileName) == FPropertyAccess::Success)
				{
					FFormatNamedArguments Args;
					Args.Add(TEXT("Identifier"), FText::FromName(OldProfileName));
					return FText::Format(LOCTEXT("InvalidProfile", "Invalid Profile {Identifier}"), Args);
				}
			}
		}
		else
		{
			return LOCTEXT("Invalid", "Invalid");
		}
	}
	// TODO: handle multiple values
	return FText();
}

#undef LOCTEXT_NAMESPACE