// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "UnrealClient.h"
#include "IPropertyTypeCustomization.h"

class IPropertyHandle;
class SWidget;

/**
 * Type customization for FZoneLaneProfileRef.
 */
class FZoneLaneProfileRefDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:

	void OnProfileComboChange(int32 Idx);
	TSharedRef<SWidget> OnGetProfileContent() const;
	FText GetCurrentProfileDesc() const;

	TSharedPtr<IPropertyHandle> NameProperty;
	TSharedPtr<IPropertyHandle> IDProperty;
	TSharedPtr<IPropertyHandle> LanesProperty;

	class IPropertyUtilities* PropUtils;
	TSharedPtr<IPropertyHandle> StructProperty;
};