// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZoneShapeComponentDetails.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ComponentVisualizer.h"
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#include "IDetailCustomNodeBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "ZoneShapeComponent.h"
#include "ZoneShapeComponentVisualizer.h"
#include "ZoneGraphSettings.h"
#include "ScopedTransaction.h"
#include "Modules/ModuleManager.h"
#include "ISettingsModule.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "ZoneShapeComponentDetails"

class FZoneShapePointDetails : public IDetailCustomNodeBuilder, public TSharedFromThis<FZoneShapePointDetails>
{
public:
	FZoneShapePointDetails(UZoneShapeComponent& InOwningComponent);

	//~ Begin IDetailCustomNodeBuilder interface
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren) override;
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
	virtual void Tick(float DeltaTime) override;
	virtual bool RequiresTick() const override { return true; }
	virtual bool InitiallyCollapsed() const override { return false; }
	virtual FName GetName() const override;
	//~ End IDetailCustomNodeBuilder interface

private:

	template <typename T>
	struct TSharedValue
	{
		TSharedValue() : bInitialized(false) {}

		void Reset()
		{
			bInitialized = false;
		}

		void Add(T InValue)
		{
			if (!bInitialized)
			{
				Value = InValue;
				bInitialized = true;
			}
			else
			{
				if (Value.IsSet() && InValue != Value.GetValue()) { Value.Reset(); }
			}
		}

		TOptional<T> Value;
		bool bInitialized;
	};

	struct FSharedVectorValue
	{
		FSharedVectorValue() : bInitialized(false) {}

		void Reset()
		{
			bInitialized = false;
		}

		bool IsValid() const { return bInitialized; }

		void Add(const FVector& V)
		{
			if (!bInitialized)
			{
				X = V.X;
				Y = V.Y;
				Z = V.Z;
				bInitialized = true;
			}
			else
			{
				if (X.IsSet() && V.X != X.GetValue()) { X.Reset(); }
				if (Y.IsSet() && V.Y != Y.GetValue()) { Y.Reset(); }
				if (Z.IsSet() && V.Z != Z.GetValue()) { Z.Reset(); }
			}
		}

		TOptional<float> X;
		TOptional<float> Y;
		TOptional<float> Z;
		bool bInitialized;
	};

	EVisibility IsVisible() const { return (SelectedPoints.Num() > 0) ? EVisibility::Visible : EVisibility::Collapsed; }
	EVisibility IsLaneProfileVisible() const;
	EVisibility IsInnerTurnRadiusVisible() const;
	EVisibility IsHidden() const { return (SelectedPoints.Num() == 0) ? EVisibility::Visible : EVisibility::Collapsed; }
	bool IsOnePointSelected() const { return SelectedPoints.Num() == 1; }
	TOptional<float> GetPositionX() const { return Position.X; }
	TOptional<float> GetPositionY() const { return Position.Y; }
	TOptional<float> GetPositionZ() const { return Position.Z; }
	TOptional<float> GetInControlPointX() const { return InControlPoint.X; }
	TOptional<float> GetInControlPointY() const { return InControlPoint.Y; }
	TOptional<float> GetInControlPointZ() const { return InControlPoint.Z; }
	TOptional<float> GetOutControlPointX() const { return OutControlPoint.X; }
	TOptional<float> GetOutControlPointY() const { return OutControlPoint.Y; }
	TOptional<float> GetOutControlPointZ() const { return OutControlPoint.Z; }
	TOptional<float> GetRotationRoll() const { return RotationRoll.Value; }
	void OnSetPosition(float NewValue, ETextCommit::Type CommitInfo, int32 Axis);
	void OnSetInControlPoint(float NewValue, ETextCommit::Type CommitInfo, int32 Axis);
	void OnSetOutControlPoint(float NewValue, ETextCommit::Type CommitInfo, int32 Axis);
	void OnSetRotationRoll(float NewValue, ETextCommit::Type CommitInfo);
	FText GetPointType() const;
	void OnPointTypeChange(FZoneShapePointType NewType);
	TSharedRef<SWidget> OnGetPointTypeContent() const;
	void OnLaneProfileComboChange(int32 Idx);
	TSharedRef<SWidget> OnGetLaneProfileContent() const;
	FText GetLaneProfile() const;
	ECheckBoxState GetReverseLaneProfile() const;
	void SetReverseLaneProfile(ECheckBoxState NewCheckedState);
	TOptional<float> GetInnerTurnRadius() const { return InnerTurnRadius.Value; }
	void OnSetInnerTurnRadius(float NewValue, ETextCommit::Type CommitInfo);

	void OnLaneConnectionRestrictionsComboChange(const EZoneShapeLaneConnectionRestrictions Value);
	TSharedRef<SWidget> OnGetLaneConnectionRestrictionsContent() const;
	FText GetLaneConnectionRestrictions() const;
	bool OnIsLaneConnectionRestrictionsSet(const EZoneShapeLaneConnectionRestrictions Value) const;

	void UpdateValues();

	UZoneShapeComponent* ShapeComp;
	TSet<int32> SelectedPoints;

	FSharedVectorValue Position;
	FSharedVectorValue InControlPoint;
	FSharedVectorValue OutControlPoint;
	TSharedValue<float> RotationRoll;
	TSharedValue<float> InnerTurnRadius;
	TSharedValue<FZoneShapePointType> PointType;
	TSharedValue<uint8> LaneProfile;
	TSharedValue<bool> ReverseLaneProfile;
	TSharedValue<int32> LaneConnectionRestrictions;
	int32 LanePointCount;

	TSharedPtr<FZoneShapeComponentVisualizer> ShapeCompVisualizer;
	FProperty* ShapePointsProperty;
	FProperty* ShapePerPointLaneProfilesProperty;
	FSimpleDelegate OnRegenerateChildren;
};

FZoneShapePointDetails::FZoneShapePointDetails(UZoneShapeComponent& InOwningComponent)
	: ShapeComp(nullptr)
	, LanePointCount(0)
{
	TSharedPtr<FComponentVisualizer> Visualizer = GUnrealEd->FindComponentVisualizer(InOwningComponent.GetClass());
	ShapeCompVisualizer = StaticCastSharedPtr<FZoneShapeComponentVisualizer>(Visualizer);
	check(ShapeCompVisualizer.IsValid());

	ShapePointsProperty = FindFProperty<FProperty>(UZoneShapeComponent::StaticClass(), TEXT("Points")); // Cant use GET_MEMBER_NAME_CHECKED(UZoneShapeComponent, Points)) on private member :(
	ShapePerPointLaneProfilesProperty = FindFProperty<FProperty>(UZoneShapeComponent::StaticClass(), TEXT("PerPointLaneProfiles")); // Ditto GET_MEMBER_NAME_CHECKED(UZoneShapeComponent, PerPointLaneProfiles))

	ShapeComp = &InOwningComponent;
}

void FZoneShapePointDetails::SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren)
{
	OnRegenerateChildren = InOnRegenerateChildren;
}

void FZoneShapePointDetails::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
}

void FZoneShapePointDetails::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	// Message which is shown when no points are selected
	ChildrenBuilder.AddCustomRow(LOCTEXT("NoneSelected", "None selected"))
		.Visibility(TAttribute<EVisibility>(this, &FZoneShapePointDetails::IsHidden))
		[
			SNew(SBox)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NoPointsSelected", "No points are selected."))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];

	// Position
	ChildrenBuilder.AddCustomRow(LOCTEXT("Position", "Position"))
		.Visibility(TAttribute<EVisibility>(this, &FZoneShapePointDetails::IsVisible))
		.NameContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Position", "Position"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MinDesiredWidth(375.0f)
		.MaxDesiredWidth(375.0f)
		[
			SNew(SVectorInputBox)
			.X(this, &FZoneShapePointDetails::GetPositionX)
			.Y(this, &FZoneShapePointDetails::GetPositionY)
			.Z(this, &FZoneShapePointDetails::GetPositionZ)
			.AllowSpin(false)
			.OnXCommitted(this, &FZoneShapePointDetails::OnSetPosition, 0)
			.OnYCommitted(this, &FZoneShapePointDetails::OnSetPosition, 1)
			.OnZCommitted(this, &FZoneShapePointDetails::OnSetPosition, 2)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

	// InControlPoint
	ChildrenBuilder.AddCustomRow(LOCTEXT("InControlPoint", "In Control Point"))
		.Visibility(TAttribute<EVisibility>(this, &FZoneShapePointDetails::IsVisible))
		.NameContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("InControlPoint", "In Control Point"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MinDesiredWidth(375.0f)
		.MaxDesiredWidth(375.0f)
		[
			SNew(SVectorInputBox)
			.X(this, &FZoneShapePointDetails::GetInControlPointX)
			.Y(this, &FZoneShapePointDetails::GetInControlPointY)
			.Z(this, &FZoneShapePointDetails::GetInControlPointZ)
			.AllowSpin(false)
			.OnXCommitted(this, &FZoneShapePointDetails::OnSetInControlPoint, 0)
			.OnYCommitted(this, &FZoneShapePointDetails::OnSetInControlPoint, 1)
			.OnZCommitted(this, &FZoneShapePointDetails::OnSetInControlPoint, 2)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

	// OutControlPoint
	ChildrenBuilder.AddCustomRow(LOCTEXT("OutControlPoint", "Out Control Point"))
		.Visibility(TAttribute<EVisibility>(this, &FZoneShapePointDetails::IsVisible))
		.NameContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("OutControlPoint", "Out Control Point"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MinDesiredWidth(375.0f)
		.MaxDesiredWidth(375.0f)
		[
			SNew(SVectorInputBox)
			.X(this, &FZoneShapePointDetails::GetOutControlPointX)
			.Y(this, &FZoneShapePointDetails::GetOutControlPointY)
			.Z(this, &FZoneShapePointDetails::GetOutControlPointZ)
			.AllowSpin(false)
			.OnXCommitted(this, &FZoneShapePointDetails::OnSetOutControlPoint, 0)
			.OnYCommitted(this, &FZoneShapePointDetails::OnSetOutControlPoint, 1)
			.OnZCommitted(this, &FZoneShapePointDetails::OnSetOutControlPoint, 2)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

	// RotationRoll
	ChildrenBuilder.AddCustomRow(LOCTEXT("RotationRoll", "Roll"))
		.Visibility(TAttribute<EVisibility>(this, &FZoneShapePointDetails::IsVisible))
		.NameContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("RotationRoll", "Roll"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MinDesiredWidth(375.0f)
		.MaxDesiredWidth(375.0f)
		[
			SNew(SNumericEntryBox<float>)
			.Value(this, &FZoneShapePointDetails::GetRotationRoll)
			.OnValueCommitted(this, &FZoneShapePointDetails::OnSetRotationRoll)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];

	// InnerTurnRadius
	ChildrenBuilder.AddCustomRow(LOCTEXT("InnerTurnRadius", "Inner Turn Radius"))
		.Visibility(TAttribute<EVisibility>(this, &FZoneShapePointDetails::IsInnerTurnRadiusVisible))
		.NameContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("InnerTurnRadius", "Inner Turn Radius"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MinDesiredWidth(375.0f)
		.MaxDesiredWidth(375.0f)
		[
			SNew(SNumericEntryBox<float>)
			.Value(this, &FZoneShapePointDetails::GetInnerTurnRadius)
			.OnValueCommitted(this, &FZoneShapePointDetails::OnSetInnerTurnRadius)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		];
	
	// Type
	ChildrenBuilder.AddCustomRow(LOCTEXT("Type", "Type"))
		.Visibility(TAttribute<EVisibility>(this, &FZoneShapePointDetails::IsVisible))
		.NameContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Type", "Type"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MinDesiredWidth(125.0f)
		.MaxDesiredWidth(125.0f)
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &FZoneShapePointDetails::OnGetPointTypeContent)
			.ContentPadding(FMargin(6.0f, 2.0f))
			.ButtonContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(this, &FZoneShapePointDetails::GetPointType)
			]
		];

	// Lane profile
	ChildrenBuilder.AddCustomRow(LOCTEXT("LaneProfile", "Lane Profile"))
		.Visibility(TAttribute<EVisibility>(this, &FZoneShapePointDetails::IsLaneProfileVisible))
		.NameContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("LaneProfile", "Lane Profile"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MinDesiredWidth(250.0f)
		.MaxDesiredWidth(250.0f)
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &FZoneShapePointDetails::OnGetLaneProfileContent)
			.ContentPadding(FMargin(6.0f, 2.0f))
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(this, &FZoneShapePointDetails::GetLaneProfile)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];

	// Reverse profile
	ChildrenBuilder.AddCustomRow(LOCTEXT("ReverseLaneProfile", "Reverse Lane Profile"))
		.Visibility(TAttribute<EVisibility>(this, &FZoneShapePointDetails::IsLaneProfileVisible))
		.NameContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ReverseLaneProfile", "Reverse Lane Profile"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MinDesiredWidth(250.0f)
		.MaxDesiredWidth(250.0f)
		[
			SNew(SCheckBox)
			.IsChecked(this, &FZoneShapePointDetails::GetReverseLaneProfile)
			.OnCheckStateChanged(this, &FZoneShapePointDetails::SetReverseLaneProfile)
		];

	// Restriction flags
	ChildrenBuilder.AddCustomRow(LOCTEXT("LaneConnectionRestrictions", "Connection Restrictions"))
		.Visibility(TAttribute<EVisibility>(this, &FZoneShapePointDetails::IsLaneProfileVisible))
		.NameContent()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("LaneConnectionRestrictions", "Connection Restrictions"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		.MinDesiredWidth(250.0f)
		.MaxDesiredWidth(250.0f)
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &FZoneShapePointDetails::OnGetLaneConnectionRestrictionsContent)
			.ContentPadding(FMargin(6.0f, 2.0f))
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(this, &FZoneShapePointDetails::GetLaneConnectionRestrictions)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];
}

void FZoneShapePointDetails::Tick(float DeltaTime)
{
	UpdateValues();
}

void FZoneShapePointDetails::UpdateValues()
{
	check(ShapeCompVisualizer.IsValid());

	bool bNeedsRebuild = false;

	// Note: this can potentially return us a selection from another zone shape.
	// We need to allow this to cooperate with Blueprint editor.
	const TSet<int32>& NewSelectedPoints = ShapeCompVisualizer->GetSelectedPoints();
	if (NewSelectedPoints.Num() != SelectedPoints.Num())
	{
		bNeedsRebuild = true;
	}
	SelectedPoints = NewSelectedPoints;

	// Cache values to be shown by the details customization.
	// An unset optional value represents 'multiple values' (in the case where multiple points are selected).
	Position.Reset();
	InControlPoint.Reset();
	OutControlPoint.Reset();
	RotationRoll.Reset();
	InnerTurnRadius.Reset();
	PointType.Reset();
	LaneProfile.Reset();
	ReverseLaneProfile.Reset();
	LaneConnectionRestrictions.Reset();

	if (ShapeComp)
	{
		TConstArrayView<FZoneShapePoint> ShapePoints = ShapeComp->GetPoints();

		LanePointCount = 0;
		for (int32 Index : SelectedPoints)
		{
			if (Index >= 0 && Index < ShapePoints.Num())
			{
				const FZoneShapePoint& Point = ShapePoints[Index];
				Position.Add(Point.Position);
				InControlPoint.Add(Point.GetInControlPoint());
				OutControlPoint.Add(Point.GetOutControlPoint());
				RotationRoll.Add(Point.Rotation.Roll);
				InnerTurnRadius.Add(Point.InnerTurnRadius);
				PointType.Add(Point.Type);
				LaneProfile.Add(Point.LaneProfile);
				if (Point.Type == FZoneShapePointType::LaneProfile)
				{
					LanePointCount++;
				}
				ReverseLaneProfile.Add(Point.bReverseLaneProfile);
				LaneConnectionRestrictions.Add(int32(Point.LaneConnectionRestrictions));
			}
		}
	}

	if (bNeedsRebuild)
	{
		OnRegenerateChildren.ExecuteIfBound();
	}
}

FName FZoneShapePointDetails::GetName() const
{
	static const FName Name("ZoneShapePointDetails");
	return Name;
}

EVisibility FZoneShapePointDetails::IsLaneProfileVisible() const
{
	if (!ShapeComp || ShapeComp->GetShapeType() != FZoneShapeType::Polygon)
	{
		return EVisibility::Collapsed;
	}

	if (SelectedPoints.Num() > 0 && LanePointCount > 0)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

void FZoneShapePointDetails::OnSetPosition(float NewValue, ETextCommit::Type CommitInfo, int32 Axis)
{
	if (!ShapeComp)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("SetZoneShapePointPosition", "Set Zone Shape point position"));
	ShapeComp->Modify();

	TArray<FZoneShapePoint>& ShapePoints = ShapeComp->GetMutablePoints();

	for (int32 Index : SelectedPoints)
	{
		FZoneShapePoint& Point = ShapePoints[Index];
		const float Delta = NewValue - Point.Position[Axis];
		Point.Position[Axis] = NewValue;
	}

	ShapeComp->UpdateShape();
	FComponentVisualizer::NotifyPropertyModified(ShapeComp, ShapePointsProperty);
	UpdateValues();
	GEditor->RedrawLevelEditingViewports(true);
}

void FZoneShapePointDetails::OnSetInControlPoint(float NewValue, ETextCommit::Type CommitInfo, int32 Axis)
{
	if (!ShapeComp)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("SetControlPoint", "Set Zone Shape control point position"));
	ShapeComp->Modify();

	TArray<FZoneShapePoint>& ShapePoints = ShapeComp->GetMutablePoints();

	for (int32 Index : SelectedPoints)
	{
		FZoneShapePoint& Point = ShapePoints[Index];
		if (Point.Type == FZoneShapePointType::Bezier || Point.Type == FZoneShapePointType::LaneProfile)
		{
			FVector ControlPoint = Point.GetInControlPoint();
			ControlPoint[Axis] = NewValue;
			Point.SetInControlPoint(ControlPoint);
		}
	}

	ShapeComp->UpdateShape();
	FComponentVisualizer::NotifyPropertyModified(ShapeComp, ShapePointsProperty);
	UpdateValues();
	GEditor->RedrawLevelEditingViewports(true);
}

void FZoneShapePointDetails::OnSetOutControlPoint(float NewValue, ETextCommit::Type CommitInfo, int32 Axis)
{
	if (!ShapeComp)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("SetControlPoint", "Set Zone Shape control point position"));
	ShapeComp->Modify();

	TArray<FZoneShapePoint>& ShapePoints = ShapeComp->GetMutablePoints();

	for (int32 Index : SelectedPoints)
	{
		FZoneShapePoint& Point = ShapePoints[Index];
		if (Point.Type == FZoneShapePointType::Bezier || Point.Type == FZoneShapePointType::LaneProfile)
		{
			FVector ControlPoint = Point.GetInControlPoint();
			ControlPoint[Axis] = NewValue;
			Point.SetOutControlPoint(ControlPoint);
		}
	}

	ShapeComp->UpdateShape();
	FComponentVisualizer::NotifyPropertyModified(ShapeComp, ShapePointsProperty);
	UpdateValues();
	GEditor->RedrawLevelEditingViewports(true);
}

void FZoneShapePointDetails::OnSetRotationRoll(float NewValue, ETextCommit::Type CommitInfo)
{
	if (!ShapeComp)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("SetRotationRoll", "Set Zone Shape rotation roll"));
	ShapeComp->Modify();

	TArray<FZoneShapePoint>& ShapePoints = ShapeComp->GetMutablePoints();

	for (int32 Index : SelectedPoints)
	{
		FZoneShapePoint& Point = ShapePoints[Index];
		Point.Rotation.Roll = NewValue;
	}

	ShapeComp->UpdateShape();
	FComponentVisualizer::NotifyPropertyModified(ShapeComp, ShapePointsProperty);
	UpdateValues();
	GEditor->RedrawLevelEditingViewports(true);
}

void FZoneShapePointDetails::OnSetInnerTurnRadius(float NewValue, ETextCommit::Type CommitInfo)
{
	if (!ShapeComp)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("SetInnerTurnRadius", "Set Zone Shape inner turn radius"));
	ShapeComp->Modify();

	TArray<FZoneShapePoint>& ShapePoints = ShapeComp->GetMutablePoints();

	for (int32 Index : SelectedPoints)
	{
		FZoneShapePoint& Point = ShapePoints[Index];
		Point.InnerTurnRadius = NewValue;
	}

	ShapeComp->UpdateShape();
	FComponentVisualizer::NotifyPropertyModified(ShapeComp, ShapePointsProperty);
	UpdateValues();
	GEditor->RedrawLevelEditingViewports(true);
}

EVisibility FZoneShapePointDetails::IsInnerTurnRadiusVisible() const
{
	if (ShapeComp
		&& SelectedPoints.Num() > 0
		&& ShapeComp->GetShapeType() == FZoneShapeType::Polygon
		&& ShapeComp->GetPolygonRoutingType() == EZoneShapePolygonRoutingType::Arcs
		&& PointType.Value.Get(FZoneShapePointType::Sharp) == FZoneShapePointType::LaneProfile)
	{
		return EVisibility::Visible;
	}
	
	return EVisibility::Collapsed;
}

FText FZoneShapePointDetails::GetPointType() const
{
	if (PointType.Value.IsSet())
	{
		UEnum* ShapePointTypeEnum = StaticEnum<FZoneShapePointType>();
		FZoneShapePointType Type = PointType.Value.Get(FZoneShapePointType::Sharp);
		return ShapePointTypeEnum->GetDisplayNameTextByValue((int64)Type);
	}

	return LOCTEXT("MultipleTypes", "Multiple Types");
}


void FZoneShapePointDetails::OnPointTypeChange(FZoneShapePointType NewType)
{
	if (!ShapeComp)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("SetPointType", "Set Zone Shape point type"));
	ShapeComp->Modify();

	TArray<FZoneShapePoint>& ShapePoints = ShapeComp->GetMutablePoints();

	for (int32 Index : SelectedPoints)
	{
		FZoneShapePoint& Point = ShapePoints[Index];
		if (Point.Type != NewType)
		{
			FZoneShapePointType OldType = Point.Type;
			Point.Type = NewType;
			if (Point.Type == FZoneShapePointType::Sharp)
			{
				Point.TangentLength = 0.0f;
			}
			else if (OldType == FZoneShapePointType::Sharp)
			{
				if (Point.Type == FZoneShapePointType::Bezier || Point.Type == FZoneShapePointType::LaneProfile)
				{
					// Initialize Bezier points with auto tangents.
					ShapeComp->UpdatePointRotationAndTangent(Index);
				}
			}
			else if (OldType == FZoneShapePointType::LaneProfile && Point.Type != FZoneShapePointType::LaneProfile)
			{
				// Change forward to point along tangent.
				Point.Rotation.Yaw -= 90.0f;
			}
			else if (OldType != FZoneShapePointType::LaneProfile && Point.Type == FZoneShapePointType::LaneProfile)
			{
				// Change forward to point inside the shape.
				Point.Rotation.Yaw += 90.0f;
			}
		}
	}

	ShapeComp->UpdateShape();
	FComponentVisualizer::NotifyPropertyModified(ShapeComp, ShapePointsProperty);
	UpdateValues();
	GEditor->RedrawLevelEditingViewports(true);
}

TSharedRef<SWidget> FZoneShapePointDetails::OnGetPointTypeContent() const
{
	FMenuBuilder MenuBuilder(true, NULL);

	UEnum* ShapePointTypeEnum = StaticEnum<FZoneShapePointType>();
	check(ShapePointTypeEnum);
	for (int32 EnumIndex = 0; EnumIndex < ShapePointTypeEnum->NumEnums() - 1; EnumIndex++)
	{
		FZoneShapePointType Value = (FZoneShapePointType)ShapePointTypeEnum->GetValueByIndex(EnumIndex);
		if (ShapeComp && ShapeComp->GetShapeType() != FZoneShapeType::Polygon && Value == FZoneShapePointType::LaneProfile)
		{
			// Do not allow to set lane point type on splines.
			continue;
		}

		FUIAction SetTypeAction(FExecuteAction::CreateSP(const_cast<FZoneShapePointDetails*>(this), &FZoneShapePointDetails::OnPointTypeChange, Value));
		MenuBuilder.AddMenuEntry(ShapePointTypeEnum->GetDisplayNameTextByIndex(EnumIndex), TAttribute<FText>(), FSlateIcon(), SetTypeAction);
		MenuBuilder.AddMenuSeparator();
	}

	return MenuBuilder.MakeWidget();
}

void FZoneShapePointDetails::OnLaneProfileComboChange(int32 Idx)
{
	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
	if (Idx == -1)
	{
		// Goto settings to create new profile
		FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer(ZoneGraphSettings->GetContainerName(), ZoneGraphSettings->GetCategoryName(), ZoneGraphSettings->GetSectionName());
		return;
	}

	if (!ShapeComp)
	{
		return;
	}

	const TArray<FZoneLaneProfile>& LaneProfiles = ZoneGraphSettings->GetLaneProfiles();

	const FScopedTransaction Transaction(LOCTEXT("SetLaneProfile", "Set Zone Shape lane profile"));
	ShapeComp->Modify();

	TArray<FZoneShapePoint>& ShapePoints = ShapeComp->GetMutablePoints();

	if (Idx == -2)
	{
		// Inherit
		for (int32 Index : SelectedPoints)
		{
			FZoneShapePoint& Point = ShapePoints[Index];
			Point.LaneProfile = FZoneShapePoint::InheritLaneProfile;
		}
	}
	else if (Idx >= 0 && Idx < LaneProfiles.Num())
	{
		const FZoneLaneProfile& NewLaneProfile = LaneProfiles[Idx];
		int32 ProfileIndex = ShapeComp->AddUniquePerPointLaneProfile(FZoneLaneProfileRef(NewLaneProfile));
		if (ProfileIndex != INDEX_NONE)
		{
			for (int32 Index : SelectedPoints)
			{
				FZoneShapePoint& Point = ShapePoints[Index];
				if (Point.Type == FZoneShapePointType::LaneProfile)
				{
					Point.LaneProfile = (uint8)ProfileIndex;
				}
			}
		}
	}

	ShapeComp->CompactPerPointLaneProfiles();

	ShapeComp->UpdateShape();
	FComponentVisualizer::NotifyPropertyModified(ShapeComp, ShapePointsProperty);
	UpdateValues();
	GEditor->RedrawLevelEditingViewports(true);
}

TSharedRef<SWidget> FZoneShapePointDetails::OnGetLaneProfileContent() const
{
	FMenuBuilder MenuBuilder(true, NULL);
	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();

	FUIAction NewItemAction(FExecuteAction::CreateSP(const_cast<FZoneShapePointDetails*>(this), &FZoneShapePointDetails::OnLaneProfileComboChange, -1));
	MenuBuilder.AddMenuEntry(LOCTEXT("CreateOrEditLaneProfile", "Create or Edit Lane Profile..."), TAttribute<FText>(), FSlateIcon(), NewItemAction);
	MenuBuilder.AddMenuSeparator();

	FUIAction CustomAction(FExecuteAction::CreateSP(const_cast<FZoneShapePointDetails*>(this), &FZoneShapePointDetails::OnLaneProfileComboChange, -2));
	MenuBuilder.AddMenuEntry(LOCTEXT("InheritFromShape", "Inherit from Shape"), TAttribute<FText>(), FSlateIcon(), CustomAction);
	MenuBuilder.AddMenuSeparator();

	const TArray<FZoneLaneProfile>& LaneProfiles = ZoneGraphSettings->GetLaneProfiles();
	for (int32 Index = 0; Index < LaneProfiles.Num(); Index++)
	{
		FUIAction ItemAction(FExecuteAction::CreateSP(const_cast<FZoneShapePointDetails*>(this), &FZoneShapePointDetails::OnLaneProfileComboChange, (int)Index));
		MenuBuilder.AddMenuEntry(FText::FromName(LaneProfiles[Index].Name), TAttribute<FText>(), FSlateIcon(), ItemAction);
	}

	return MenuBuilder.MakeWidget();
}

FText FZoneShapePointDetails::GetLaneProfile() const
{
	if (!ShapeComp)
	{
		return LOCTEXT("Invalid", "Invalid");
	}

	if (LaneProfile.Value.IsSet())
	{
		const uint8 LaneProfileIndex = LaneProfile.Value.Get(FZoneShapePoint::InheritLaneProfile);

		if (LaneProfileIndex == FZoneShapePoint::InheritLaneProfile)
		{
			return LOCTEXT("InheritFromShape", "Inherit from Shape");
		}
		else
		{
			FZoneLaneProfileRef ProfileRef;
			TConstArrayView<FZoneLaneProfileRef> PerPointLaneProfiles = ShapeComp->GetPerPointLaneProfiles();
			if (ensure(LaneProfileIndex < PerPointLaneProfiles.Num()))
			{
				ProfileRef = PerPointLaneProfiles[LaneProfileIndex];
			}

			const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
			if (const FZoneLaneProfile* Profile = ZoneGraphSettings->GetLaneProfileByRef(ProfileRef))
			{
				return FText::FromName(Profile->Name);
			}
			else
			{
				FFormatNamedArguments Args;
				Args.Add(TEXT("Identifier"), FText::FromName(ProfileRef.Name));
				return FText::Format(LOCTEXT("InvalidProfile", "Invalid Profile {Identifier}"), Args);
			}
		}
	}

	return LOCTEXT("MultipleValues", "Multiple Values");
}

ECheckBoxState FZoneShapePointDetails::GetReverseLaneProfile() const
{
	if (ReverseLaneProfile.Value.IsSet())
	{
		static const bool bDefaultReverseLaneProfile = false;
		return ReverseLaneProfile.Value.Get(bDefaultReverseLaneProfile) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Undetermined;
}

void FZoneShapePointDetails::SetReverseLaneProfile(ECheckBoxState NewCheckedState)
{
	const bool bState = NewCheckedState == ECheckBoxState::Checked;

	if (!ShapeComp)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("SetReverLaneProfile", "Set Zone Shape reverse lane profile"));
	ShapeComp->Modify();

	TArray<FZoneShapePoint>& ShapePoints = ShapeComp->GetMutablePoints();

	// Inherit
	for (int32 Index : SelectedPoints)
	{
		FZoneShapePoint& Point = ShapePoints[Index];
		Point.bReverseLaneProfile = bState;
	}

	ShapeComp->UpdateShape();
	FComponentVisualizer::NotifyPropertyModified(ShapeComp, ShapePointsProperty);
	UpdateValues();
	GEditor->RedrawLevelEditingViewports(true);
}

void FZoneShapePointDetails::OnLaneConnectionRestrictionsComboChange(const EZoneShapeLaneConnectionRestrictions Value)
{
	if (!ShapeComp)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("SetLaneConnectionRestrictions", "Set Zone Shape generation flags"));
	ShapeComp->Modify();

	TArray<FZoneShapePoint>& ShapePoints = ShapeComp->GetMutablePoints();

	if (Value == EZoneShapeLaneConnectionRestrictions::None)
	{
		// None
		for (int32 Index : SelectedPoints)
		{
			FZoneShapePoint& Point = ShapePoints[Index];
			Point.LaneConnectionRestrictions = int32(EZoneShapeLaneConnectionRestrictions::None);
		}
	}
	else
	{
		// Toggle
		for (int32 Index : SelectedPoints)
		{
			FZoneShapePoint& Point = ShapePoints[Index];
			Point.LaneConnectionRestrictions ^= int32(Value);
		}
	}

	ShapeComp->UpdateShape();
	FComponentVisualizer::NotifyPropertyModified(ShapeComp, ShapePointsProperty);
	UpdateValues();
	GEditor->RedrawLevelEditingViewports(true);
}

TSharedRef<SWidget> FZoneShapePointDetails::OnGetLaneConnectionRestrictionsContent() const
{
	UEnum* Enum = StaticEnum<EZoneShapeLaneConnectionRestrictions>();
	check(Enum);

	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection*/false, /*commnadlist*/nullptr);

	for (int32 Index = 0; Index < Enum->NumEnums() - 1; Index++) // -1 to prevent MAX from showing up.
	{
		if (Enum->HasMetaData(TEXT("Hidden"), Index))
		{
			continue;
		}

		const EZoneShapeLaneConnectionRestrictions Value = (EZoneShapeLaneConnectionRestrictions)Enum->GetValueByIndex(Index);
		if (Value == EZoneShapeLaneConnectionRestrictions::None)
		{
			continue;
		}

		FUIAction ValueAction
		(
			FExecuteAction::CreateSP(const_cast<FZoneShapePointDetails*>(this), &FZoneShapePointDetails::OnLaneConnectionRestrictionsComboChange, Value),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(const_cast<FZoneShapePointDetails*>(this), &FZoneShapePointDetails::OnIsLaneConnectionRestrictionsSet, Value)
		);
		MenuBuilder.AddMenuEntry(Enum->GetDisplayNameTextByIndex(Index), TAttribute<FText>(), FSlateIcon(), ValueAction, FName(), EUserInterfaceActionType::Check);
	}

	return MenuBuilder.MakeWidget();
}

bool FZoneShapePointDetails::OnIsLaneConnectionRestrictionsSet(const EZoneShapeLaneConnectionRestrictions Value) const
{
	const int32 Flags = LaneConnectionRestrictions.Value.Get(0);
	return (Flags & int32(Value)) != 0;
}

FText FZoneShapePointDetails::GetLaneConnectionRestrictions() const
{
	if (!ShapeComp)
	{
		return LOCTEXT("Invalid", "Invalid");
	}

	UEnum* Enum = StaticEnum<EZoneShapeLaneConnectionRestrictions>();
	check(Enum);

	if (LaneConnectionRestrictions.Value.IsSet())
	{
		const int32 Flags = LaneConnectionRestrictions.Value.Get(0);

		TArray<FText> Names;
		for (int32 Index = 0; Index < Enum->NumEnums() - 1; Index++) // -1 to prevent MAX from showing up.
		{
			if (Enum->HasMetaData(TEXT("Hidden"), Index))
			{
				continue;
			}
			const int32 Value = (int32)Enum->GetValueByIndex(Index);
			if ((Flags & Value) != 0)
			{
				Names.Add(Enum->GetDisplayNameTextByValue((int64)Value));
			}
		}
		if (Names.Num() == 0)
		{
			return LOCTEXT("None", "None");
		}
		else
		{
			if (Names.Num() > 2)
			{
				Names.SetNum(2);
				Names.Add(FText::FromString(TEXT("...")));
			}
			return FText::Join(FText::FromString(TEXT(", ")), Names);
		}
	}

	return LOCTEXT("MultipleValues", "Multiple Values");
}

////////////////////////////////////

TSharedRef<IDetailCustomization> FZoneShapeComponentDetails::MakeInstance()
{
	return MakeShareable(new FZoneShapeComponentDetails);
}

void FZoneShapeComponentDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	// Hide the SplineCurves property
	TSharedPtr<IPropertyHandle> PointsProperty = DetailBuilder.GetProperty(TEXT("Points")); // Cant use GET_MEMBER_NAME_CHECKED(UZoneShapeComponent, Points)) on private member :(
	PointsProperty->MarkHiddenByCustomization();

	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);

	if (ObjectsBeingCustomized.Num() == 1)
	{
		if (UZoneShapeComponent* ShapeComp = Cast<UZoneShapeComponent>(ObjectsBeingCustomized[0]))
		{
			// Set the spline points details as important in order to have it on top 
			IDetailCategoryBuilder& Category = DetailBuilder.EditCategory("Selected Points", FText::GetEmpty(), ECategoryPriority::Important);
			TSharedRef<FZoneShapePointDetails> PointDetails = MakeShareable(new FZoneShapePointDetails(*ShapeComp));
			Category.AddCustomBuilder(PointDetails);
		}
	}
}

#undef LOCTEXT_NAMESPACE
