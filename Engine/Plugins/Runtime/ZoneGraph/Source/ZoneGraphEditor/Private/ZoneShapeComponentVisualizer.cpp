// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZoneShapeComponentVisualizer.h"
#include "CoreMinimal.h"
#include "Algo/AnyOf.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SceneView.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Styling/AppStyle.h"
#include "Editor.h"
#include "EditorViewportClient.h"
#include "EditorViewportCommands.h"
#include "LevelEditor.h"
#include "LevelEditorActions.h"
#include "ScopedTransaction.h"
#include "ActorEditorUtils.h"
#include "ZoneGraphQuery.h"
#include "ZoneGraphSubsystem.h"
#include "ZoneGraphSettings.h"
#include "ZoneShapeActor.h"
#include "ZoneShapeComponent.h"
#include "ZoneShapeUtilities.h"
#include "ZoneGraphRenderingUtilities.h"
#include "Curves/BezierUtilities.h"
#include "CanvasTypes.h"
#include "Modules/ModuleManager.h"
#include "PrimitiveDrawingUtils.h"

// Uncomment to draw additional rotation debug visualizations.
// #define ZONEGRAPH_DEBUG_ROTATIONS

#include UE_INLINE_GENERATED_CPP_BY_NAME(ZoneShapeComponentVisualizer)

IMPLEMENT_HIT_PROXY(HZoneShapeVisProxy, HComponentVisProxy);
IMPLEMENT_HIT_PROXY(HZoneShapePointProxy, HZoneShapeVisProxy);
IMPLEMENT_HIT_PROXY(HZoneShapeSegmentProxy, HZoneShapeVisProxy);
IMPLEMENT_HIT_PROXY(HZoneShapeControlPointProxy, HZoneShapeVisProxy);

#define LOCTEXT_NAMESPACE "ZoneShapeComponentVisualizer"
DEFINE_LOG_CATEGORY_STATIC(LogZoneShapeComponentVisualizer, Log, All)


namespace UE::ZoneGraph::Editor::Private
{
	double GetClockwiseAngle(const FVector& P)
	{
		return -FMath::Atan2(P.X, -P.Y);
	}

	bool ComparePoints(const FVector& P1, const FVector& P2)
	{
		return GetClockwiseAngle(P1) > GetClockwiseAngle(P2);
	}

	void SortPolygonPointsCounterclockwise(UZoneShapeComponent* PolygonShapeComp)
	{
		if (PolygonShapeComp->GetShapeType() != FZoneShapeType::Polygon)
		{
			return;
		}

		TArray<FZoneShapePoint>& Points = PolygonShapeComp->GetMutablePoints();

		// Compute the center
		FVector Center = FVector::ZeroVector;
		for (const FZoneShapePoint& Point : Points)
		{
			Center += Point.Position;
		}
		Center /= Points.Num();

		Points.Sort([Center](const FZoneShapePoint& Point1, const FZoneShapePoint& Point2) {
			return ComparePoints(Point1.Position - Center, Point2.Position - Center);
		});
	}

	FVector GetPositionOnSegment(const TArray<FZoneShapePoint>& Points, int32 SegmentIndex, float SegmentT)
	{
		const int32 NumPoints = Points.Num();
		const int32 StartPointIdx = SegmentIndex;
		const int32 EndPointIdx = (SegmentIndex + 1) % NumPoints;
		const FZoneShapePoint& StartPoint = Points[StartPointIdx];
		const FZoneShapePoint& EndPoint = Points[EndPointIdx];

		FVector StartPosition(ForceInitToZero), StartControlPoint(ForceInitToZero), EndControlPoint(ForceInitToZero), EndPosition(ForceInitToZero);
		UE::ZoneShape::Utilities::GetCubicBezierPointsFromShapeSegment(StartPoint, EndPoint, FMatrix::Identity, StartPosition, StartControlPoint, EndControlPoint, EndPosition);

		return UE::CubicBezier::Eval(StartPosition, StartControlPoint, EndControlPoint, EndPosition, SegmentT);
	}

	void SetPolygonPointLaneProfileToMatchSpline(FZoneShapePoint& Point, UZoneShapeComponent* Polygon, UZoneShapeComponent* Spline)
	{
		Point.Type = FZoneShapePointType::LaneProfile;
		const FZoneLaneProfileRef ShapeComponent0LaneProfileRef = Spline->GetCommonLaneProfile();
		const int32 ProfileIndex = Polygon->AddUniquePerPointLaneProfile(ShapeComponent0LaneProfileRef);
		if (ProfileIndex != INDEX_NONE)
		{
			Point.LaneProfile = (uint8)ProfileIndex;
		}
	}

	void SetPointPositionRotation(
		FZoneShapePoint& Point,
		const FTransform& SourceTransform,
		const FVector& TargetPointWorldPosition,
		const FVector& TargetPointWorldNormal)
	{
		Point.Position = SourceTransform.InverseTransformPosition(TargetPointWorldPosition);
		FVector Normal = SourceTransform.InverseTransformVector(TargetPointWorldNormal);
		Point.Rotation = FRotationMatrix::MakeFromX(Normal).Rotator();
	}

	void SnapConnect(
		UZoneShapeComponent* ShapeComp,
		FZoneShapePoint& DraggedPoint,
		const FTransform& SourceTransform,
		const FVector& SourceWorldNormal,
		const FVector& TargetPointWorldPosition,
		const FVector& TargetPointWorldNormal,
		double ConnectionSnapAngleCos,
		double HalfLanesTotalWidth)
	{
		// Snap point location
		UE::ZoneGraph::Editor::Private::SetPointPositionRotation(DraggedPoint, SourceTransform, TargetPointWorldPosition, TargetPointWorldNormal);

		// If the zone shape is a spline and the point type is not Bezier, setting the point rotation doesn't work.
		// An extra point is needed to align the connectors and make it connect.
		if (ShapeComp->GetShapeType() == FZoneShapeType::Spline &&
			DraggedPoint.Type != FZoneShapePointType::Bezier &&
			FVector::DotProduct(SourceWorldNormal, -TargetPointWorldNormal) <= ConnectionSnapAngleCos)
		{
			// Add extra point
			TArray<FZoneShapePoint>& Points = ShapeComp->GetMutablePoints();
			FZoneShapePoint ExtraPoint = DraggedPoint;
			ExtraPoint.Position += SourceTransform.InverseTransformVector(TargetPointWorldNormal) * HalfLanesTotalWidth;
			ExtraPoint.Rotation = DraggedPoint.Rotation;
			Points.Insert(ExtraPoint, ShapeComp->GetNumPoints() - 1);
		}

		// Update shape
		ShapeComp->UpdateShape();
	}
} // UE::ZoneGraph::Editor::Private


/** Define commands for the shape component visualizer */
class FZoneShapeComponentVisualizerCommands : public TCommands<FZoneShapeComponentVisualizerCommands>
{
public:
	FZoneShapeComponentVisualizerCommands() : TCommands <FZoneShapeComponentVisualizerCommands>
		(
			"ZoneShapeComponentVisualizer",	// Context name for fast lookup
			LOCTEXT("ZoneShapeComponentVisualizer", "Zone Shape Component Visualizer"),	// Localized context name for displaying
			FName(),	// Parent
			FAppStyle::GetAppStyleSetName()
			)
	{
	}

	virtual void RegisterCommands() override
	{
		UI_COMMAND(DeletePoint, "Delete Point(s)", "Delete the currently selected shape points.", EUserInterfaceActionType::Button, FInputChord(EKeys::Delete));
		UI_COMMAND(DuplicatePoint, "Duplicate Point(s)", "Duplicate the currently selected shape points.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(AddPoint, "Add Point Here", "Add a new shape point at the cursor location.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(SelectAll, "Select All Points", "Select all shape points.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(SetPointToSharp, "Sharp", "Set point to Sharp type", EUserInterfaceActionType::RadioButton, FInputChord());
		UI_COMMAND(SetPointToBezier, "Bezier", "Set point to Bezier type", EUserInterfaceActionType::RadioButton, FInputChord());
		UI_COMMAND(SetPointToAutoBezier, "Auto Bezier", "Set point to Auto Bezier type", EUserInterfaceActionType::RadioButton, FInputChord());
		UI_COMMAND(SetPointToLaneSegment, "Lane Segment", "Set point to Lane Segment type", EUserInterfaceActionType::RadioButton, FInputChord());
		UI_COMMAND(FocusViewportToSelection, "Focus Selected", "Moves the camera in front of the selection", EUserInterfaceActionType::Button, FInputChord(EKeys::F));
		UI_COMMAND(BreakAtPointNewActors, "Break Into Shape Actors At Point(s)", "Break the shape into multiple shape actors at the currently selected points.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(BreakAtPointNewComponents, "Break Into Shape Components At Point(s)", "Break the shape into multiple shape components at the currently selected points.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(BreakAtSegmentNewActors, "Break Into Shape Actors Here", "Break the shape into multiple shape actors at the cursor location.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(BreakAtSegmentNewComponents, "Break Into Shape Components Here", "Break the shape into multiple shape components at the cursor location.", EUserInterfaceActionType::Button, FInputChord());
	}

public:
	TSharedPtr<FUICommandInfo> DeletePoint;
	TSharedPtr<FUICommandInfo> DuplicatePoint;
	TSharedPtr<FUICommandInfo> AddPoint;
	TSharedPtr<FUICommandInfo> SelectAll;
	TSharedPtr<FUICommandInfo> SetPointToSharp;
	TSharedPtr<FUICommandInfo> SetPointToBezier;
	TSharedPtr<FUICommandInfo> SetPointToAutoBezier;
	TSharedPtr<FUICommandInfo> SetPointToLaneSegment;
	TSharedPtr<FUICommandInfo> FocusViewportToSelection;
	TSharedPtr<FUICommandInfo> BreakAtPointNewActors;
	TSharedPtr<FUICommandInfo> BreakAtPointNewComponents;
	TSharedPtr<FUICommandInfo> BreakAtSegmentNewActors;
	TSharedPtr<FUICommandInfo> BreakAtSegmentNewComponents;
};

FZoneShapeComponentVisualizer::FZoneShapeComponentVisualizer()
	: FComponentVisualizer()
	, bAllowDuplication(true)
	, DuplicateAccumulatedDrag(FVector::ZeroVector)
	, bControlPointPositionCaptured(false)
	, ControlPointPosition(FVector::ZeroVector)
{
	FZoneShapeComponentVisualizerCommands::Register();

	ShapeComponentVisualizerActions = MakeShareable(new FUICommandList);

	ShapePointsProperty = FindFProperty<FProperty>(UZoneShapeComponent::StaticClass(), TEXT("Points")); //Can't use GET_MEMBER_NAME_CHECKED(UZoneShapeComponent, Points)) on private members :(

	SelectionState = NewObject<UZoneShapeComponentVisualizerSelectionState>(GetTransientPackage(), TEXT("ZoneShapeSelectionState"), RF_Transactional);
}

void FZoneShapeComponentVisualizer::OnRegister()
{
	const auto& Commands = FZoneShapeComponentVisualizerCommands::Get();

	ShapeComponentVisualizerActions->MapAction(
		Commands.DeletePoint,
		FExecuteAction::CreateSP(this, &FZoneShapeComponentVisualizer::OnDeletePoint),
		FCanExecuteAction::CreateSP(this, &FZoneShapeComponentVisualizer::CanDeletePoint));

	ShapeComponentVisualizerActions->MapAction(
		Commands.DuplicatePoint,
		FExecuteAction::CreateSP(this, &FZoneShapeComponentVisualizer::OnDuplicatePoint),
		FCanExecuteAction::CreateSP(this, &FZoneShapeComponentVisualizer::IsPointSelectionValid));

	ShapeComponentVisualizerActions->MapAction(
		Commands.AddPoint,
		FExecuteAction::CreateSP(this, &FZoneShapeComponentVisualizer::OnAddPointToSegment),
		FCanExecuteAction::CreateSP(this, &FZoneShapeComponentVisualizer::CanAddPointToSegment));

	ShapeComponentVisualizerActions->MapAction(
		Commands.SelectAll,
		FExecuteAction::CreateSP(this, &FZoneShapeComponentVisualizer::OnSelectAllPoints),
		FCanExecuteAction::CreateSP(this, &FZoneShapeComponentVisualizer::CanSelectAllPoints));

	ShapeComponentVisualizerActions->MapAction(
		Commands.SetPointToSharp,
		FExecuteAction::CreateSP(this, &FZoneShapeComponentVisualizer::OnSetPointType, FZoneShapePointType::Sharp),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FZoneShapeComponentVisualizer::IsPointTypeSet, FZoneShapePointType::Sharp));

	ShapeComponentVisualizerActions->MapAction(
		Commands.SetPointToBezier,
		FExecuteAction::CreateSP(this, &FZoneShapeComponentVisualizer::OnSetPointType, FZoneShapePointType::Bezier),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FZoneShapeComponentVisualizer::IsPointTypeSet, FZoneShapePointType::Bezier));

	ShapeComponentVisualizerActions->MapAction(
		Commands.SetPointToAutoBezier,
		FExecuteAction::CreateSP(this, &FZoneShapeComponentVisualizer::OnSetPointType, FZoneShapePointType::AutoBezier),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FZoneShapeComponentVisualizer::IsPointTypeSet, FZoneShapePointType::AutoBezier));

	ShapeComponentVisualizerActions->MapAction(
		Commands.SetPointToLaneSegment,
		FExecuteAction::CreateSP(this, &FZoneShapeComponentVisualizer::OnSetPointType, FZoneShapePointType::LaneProfile),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FZoneShapeComponentVisualizer::IsPointTypeSet, FZoneShapePointType::LaneProfile));

	ShapeComponentVisualizerActions->MapAction(
		Commands.FocusViewportToSelection,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ExecuteExecCommand, FString(TEXT("CAMERA ALIGN ACTIVEVIEWPORTONLY")))
	);

	ShapeComponentVisualizerActions->MapAction(
		Commands.BreakAtPointNewActors,
		FExecuteAction::CreateSP(this, &FZoneShapeComponentVisualizer::OnBreakAtPointNewActors),
		FCanExecuteAction::CreateSP(this, &FZoneShapeComponentVisualizer::CanBreakAtPoint));

	ShapeComponentVisualizerActions->MapAction(
		Commands.BreakAtPointNewComponents,
		FExecuteAction::CreateSP(this, &FZoneShapeComponentVisualizer::OnBreakAtPointNewComponents),
		FCanExecuteAction::CreateSP(this, &FZoneShapeComponentVisualizer::CanBreakAtPoint));

	ShapeComponentVisualizerActions->MapAction(
		Commands.BreakAtSegmentNewActors,
		FExecuteAction::CreateSP(this, &FZoneShapeComponentVisualizer::OnBreakAtSegmentNewActors),
		FCanExecuteAction::CreateSP(this, &FZoneShapeComponentVisualizer::CanBreakAtSegment));

	ShapeComponentVisualizerActions->MapAction(
		Commands.BreakAtSegmentNewComponents,
		FExecuteAction::CreateSP(this, &FZoneShapeComponentVisualizer::OnBreakAtSegmentNewComponents),
		FCanExecuteAction::CreateSP(this, &FZoneShapeComponentVisualizer::CanBreakAtSegment));

	bool bAlign = false;
	bool bUseLineTrace = false;
	bool bUseBounds = false;
	bool bUsePivot = false;
	ShapeComponentVisualizerActions->MapAction(
		FLevelEditorCommands::Get().SnapToFloor,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::SnapToFloor_Clicked, bAlign, bUseLineTrace, bUseBounds, bUsePivot),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ActorSelected_CanExecute)
	);

	bAlign = true;
	bUseLineTrace = false;
	bUseBounds = false;
	bUsePivot = false;
	ShapeComponentVisualizerActions->MapAction(
		FLevelEditorCommands::Get().AlignToFloor,
		FExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::SnapToFloor_Clicked, bAlign, bUseLineTrace, bUseBounds, bUsePivot),
		FCanExecuteAction::CreateStatic(&FLevelEditorActionCallbacks::ActorSelected_CanExecute)
	);
}

FZoneShapeComponentVisualizer::~FZoneShapeComponentVisualizer()
{
	FZoneShapeComponentVisualizerCommands::Unregister();
}

void FZoneShapeComponentVisualizer::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (SelectionState)
	{
		Collector.AddReferencedObject(SelectionState);
	}
}
void FZoneShapeComponentVisualizer::DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	const UZoneShapeComponent* ShapeComp = Cast<const UZoneShapeComponent>(Component);
	if (!ShapeComp)
	{
		return;
	}
	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
	if (!ZoneGraphSettings)
	{
		return;
	}

	const FZoneGraphBuildSettings& BuildSettings = ZoneGraphSettings->GetBuildSettings();

	const FMatrix LocalToWorld = ShapeComp->GetComponentTransform().ToMatrixWithScale();

	// Distance culling.
	float ShapeMaxDrawDistance = MAX_flt;
	ShapeMaxDrawDistance = ZoneGraphSettings->GetShapeMaxDrawDistance();
	
	const float MaxDrawDistanceSqr = FMath::Square(ShapeMaxDrawDistance);

	// Taking into account the min and maximum drawing distance
	const FBoxSphereBounds ShapeBounds = ShapeComp->CalcBounds(ShapeComp->GetComponentTransform());
	const float DistanceSqr = FVector::DistSquared(ShapeBounds.Origin, View->ViewMatrices.GetViewOrigin());
	if (DistanceSqr > MaxDrawDistanceSqr)
	{
		return;
	}

	const UZoneShapeComponent* EditedShapeComp = GetEditedShapeComponent();
	const bool bIsActiveComponent = Component == EditedShapeComp;

	constexpr FColor NormalColor = FColor(255, 255, 255, 255);
	constexpr FColor SelectedColor = FColor(211, 93, 0, 255);
	constexpr FColor TangentColor = SelectedColor;
	
	const float GrabHandleSize = GetDefault<ULevelEditorViewportSettings>()->SelectedSplinePointSizeAdjustment + (bIsActiveComponent ? 10.0f : 0.0f);

	static constexpr float DepthBias = 0.0001f; // Little bias helps to make the lines visible when directly on top of geometry.
	static constexpr float HandlesDepthBias = 0.0002f; // A bit more than in the shape drawing, so that we get drawn on top
	static constexpr float LaneLineThickness = 2.0f;
	static constexpr float BoundaryLineThickness = 0.0f;

	TConstArrayView<FZoneShapePoint> ShapePoints = ShapeComp->GetPoints();
	check(SelectionState);

	// Lanes
	FZoneGraphStorage Zone;
	if (UZoneGraphSubsystem* ZoneGraph = UWorld::GetSubsystem<UZoneGraphSubsystem>(ShapeComp->GetWorld()))
	{
		ZoneGraph->GetBuilder().BuildSingleShape(*ShapeComp, FMatrix::Identity, Zone);
		Zone.DataHandle = FZoneGraphDataHandle(0xffff, 0xffff); // Give a valid handle so that the drawing happens correctly.
	}

	TConstArrayView<FZoneShapeConnector> Connectors = ShapeComp->GetShapeConnectors();
	TConstArrayView<FZoneShapeConnection> Connections = ShapeComp->GetConnectedShapes();

	PDI->SetHitProxy(nullptr);

	constexpr int32 ZoneIndex = 0; // We have only one zone in the storage, created above.
	constexpr bool bDrawDetails = true;
	const float ShapeAlpha = bIsActiveComponent ? 1.0f : 0.5f;
	UE::ZoneGraph::RenderingUtilities::FLaneHighlight LaneHighlight;

	// Highlight lanes that emanate from the selected point.
	if (bIsActiveComponent && ShapePoints.Num() > 0 && SelectionState->GetSelectedPoints().Num() > 0)
	{
		const int32 LastPointIndex = SelectionState->GetLastPointIndexSelected();
		if (ShapePoints.IsValidIndex(LastPointIndex))
		{
			const FZoneShapePoint& Point = ShapePoints[LastPointIndex];
			if (Point.Type == FZoneShapePointType::LaneProfile)
			{
				LaneHighlight.Position = LocalToWorld.TransformPosition(Point.Position);
				LaneHighlight.Rotation = LocalToWorld.ToQuat() * Point.Rotation.Quaternion();
				LaneHighlight.Width = Point.TangentLength;
			}
		}
	}

	// Draw boundary
	UE::ZoneGraph::RenderingUtilities::DrawZoneBoundary(Zone, ZoneIndex, PDI, LocalToWorld, BoundaryLineThickness, DepthBias, ShapeAlpha);

	// Draw Lanes
	PDI->SetHitProxy(new HZoneShapeVisProxy(Component));
	UE::ZoneGraph::RenderingUtilities::DrawZoneLanes(Zone, ZoneIndex, PDI, LocalToWorld, LaneLineThickness, DepthBias, ShapeAlpha, bDrawDetails, LaneHighlight);

	// Draw connectors
	for (int32 i = 0; i < Connectors.Num(); i++)
	{
		const FZoneShapeConnector& Connector = Connectors[i];
		const FZoneShapeConnection* Connection = i < Connections.Num() ? &Connections[i] : nullptr;
		PDI->SetHitProxy(new HZoneShapePointProxy(Component, Connector.PointIndex));
		UE::ZoneGraph::RenderingUtilities::DrawZoneShapeConnector(Connector, Connection, PDI, LocalToWorld, DepthBias);
	}

	// Segments
	if (ShapePoints.Num() > 1)
	{
		const int32 NumPoints = ShapePoints.Num();
		int32 StartIdx = ShapeComp->IsShapeClosed() ? (NumPoints - 1) : 0;
		int32 Idx = ShapeComp->IsShapeClosed() ? 0 : 1;

		TArray<FVector> CurvePoints;

		while (Idx < NumPoints)
		{
			const FZoneShapePoint& StartPoint = ShapePoints[StartIdx];
			const FZoneShapePoint& EndPoint = ShapePoints[Idx];

			FVector StartPosition(ForceInitToZero), StartControlPoint(ForceInitToZero), EndControlPoint(ForceInitToZero), EndPosition(ForceInitToZero);
			UE::ZoneShape::Utilities::GetCubicBezierPointsFromShapeSegment(StartPoint, EndPoint, LocalToWorld, StartPosition, StartControlPoint, EndControlPoint, EndPosition);

			PDI->SetHitProxy(new HZoneShapeSegmentProxy(Component, StartIdx));
			const FColor Color = (ShapeComp == EditedShapeComp && StartIdx == SelectionState->GetSelectedSegmentIndex()) ? SelectedColor : NormalColor;

			// TODO: Make this a setting or property on shape
			static constexpr float TessTolerance = 5.0f;
			CurvePoints.Reset();

			if (StartPoint.Type == FZoneShapePointType::LaneProfile)
			{
				CurvePoints.Add(LocalToWorld.TransformPosition(StartPoint.Position));
			}

			CurvePoints.Add(StartPosition);
			UE::CubicBezier::Tessellate(CurvePoints, StartPosition, StartControlPoint, EndControlPoint, EndPosition, TessTolerance);

			if (EndPoint.Type == FZoneShapePointType::LaneProfile)
			{
				CurvePoints.Add(LocalToWorld.TransformPosition(EndPoint.Position));
			}

			for (int32 i = 0; i < CurvePoints.Num() - 1; i++)
			{
				PDI->DrawLine(CurvePoints[i], CurvePoints[i + 1], Color, SDPG_Foreground, BoundaryLineThickness, HandlesDepthBias, true);
			}

			StartIdx = Idx;
			Idx++;
		}
	}

	// Draw handles on selected shapes
	if (bIsActiveComponent)
	{
		const int32 NumPoints = ShapePoints.Num();

		if (NumPoints == 0 && SelectionState->GetSelectedPoints().Num() > 0)
		{
			ChangeSelectionState(INDEX_NONE, false);
		}
		else
		{
			const TSet<int32> SelectedPointsCopy = SelectionState->GetSelectedPoints();
			for (int32 SelectedPoint : SelectedPointsCopy)
			{
				check(SelectedPoint >= 0);
				if (SelectedPoint >= NumPoints)
				{
					// Catch any keys that might not exist anymore due to the underlying component changing.
					ChangeSelectionState(SelectedPoint, true);
					continue;
				}

				const FZoneShapePoint& Point = ShapePoints[SelectedPoint];

				if (Point.Type == FZoneShapePointType::Bezier || Point.Type == FZoneShapePointType::LaneProfile)
				{
					const float TangentHandleSize = 8.0f + GetDefault<ULevelEditorViewportSettings>()->SplineTangentHandleSizeAdjustment;

					const FVector Position = LocalToWorld.TransformPosition(Point.Position);
					const FVector InControlPoint = LocalToWorld.TransformPosition(Point.GetInControlPoint());
					const FVector OutControlPoint = LocalToWorld.TransformPosition(Point.GetOutControlPoint());

					PDI->SetHitProxy(nullptr);

					PDI->DrawLine(Position, InControlPoint, TangentColor, SDPG_Foreground, 0.0, HandlesDepthBias);
					PDI->DrawLine(Position, OutControlPoint, TangentColor, SDPG_Foreground, 0.0, HandlesDepthBias);

					PDI->SetHitProxy(new HZoneShapeControlPointProxy(Component, SelectedPoint, true));
					PDI->DrawPoint(InControlPoint, TangentColor, TangentHandleSize, SDPG_Foreground);

					PDI->SetHitProxy(new HZoneShapeControlPointProxy(Component, SelectedPoint, false));
					PDI->DrawPoint(OutControlPoint, TangentColor, TangentHandleSize, SDPG_Foreground);

					PDI->SetHitProxy(nullptr);
				}
			}
		}
	}

	// Points
	for (int32 i = 0; i < ShapePoints.Num(); i++)
	{
		const FVector Point = LocalToWorld.TransformPosition(ShapePoints[i].Position);
		const FColor Color = (ShapeComp == EditedShapeComp && SelectionState->GetSelectedPoints().Contains(i)) ? SelectedColor : NormalColor;
		PDI->SetHitProxy(new HZoneShapePointProxy(Component, i));
		PDI->DrawPoint(Point, Color, GrabHandleSize, SDPG_Foreground);

#ifdef ZONEGRAPH_DEBUG_ROTATIONS
		const FRotator& Rot = ShapePoints[i].Rotation;
		const FVector Forward = LocalToWorld.TransformVector(Rot.RotateVector(FVector::ForwardVector));
		const FVector Side = LocalToWorld.TransformVector(Rot.RotateVector(FVector::RightVector));
		const FVector Up = LocalToWorld.TransformVector(Rot.RotateVector(FVector::UpVector));
		PDI->DrawLine(Point, Point + Forward * 40.0f, FColor::Red, SDPG_Foreground, 4.0f, HandlesDepthBias, true);
		PDI->DrawLine(Point, Point + Side * 40.0f, FColor::Green, SDPG_Foreground, 4.0f, HandlesDepthBias, true);
		PDI->DrawLine(Point, Point + Up * 40.0f, FColor::Blue, SDPG_Foreground, 4.0f, HandlesDepthBias, true);
#endif
	}

	if (bIsActiveComponent && (bIsAutoConnecting || bIsCreatingIntersection) && ShapePoints.IsValidIndex(SelectedPointForConnecting))
	{
		const FZoneShapePoint& DraggedPoint = ShapePoints[SelectedPointForConnecting];
		const FVector Center = ShapeComp->GetComponentTransform().TransformPosition(DraggedPoint.Position);
		const FTransform Transform(FQuat::Identity, Center);
		constexpr FColor IndicatorColor = FColor(255, 192, 32, 255);
		constexpr FColor InnerIndicatorColor = FColor(192, 128, 16, 255);

		double IndicatorRadius = 0.0;
		double IndicatorInnerRadius = 0.0;

		if (bIsCreatingIntersection)
		{
			if (UZoneShapeComponent* TargetShapeComponent = CreateIntersectionState.WeakTargetShapeComponent.Get())
			{
				const FTransform& TargetShapeTransform = TargetShapeComponent->GetComponentTransform();

				// Draw X at the indicative location where the intersection will be build.
				constexpr double MarkerHalfSize = 10.0;
				const FVector AxisX = TargetShapeTransform.GetUnitAxis(EAxis::X);
				const FVector AxisY = TargetShapeTransform.GetUnitAxis(EAxis::Y);

				PDI->DrawLine(
					CreateIntersectionState.PreviewLocation - AxisX * MarkerHalfSize - AxisY * MarkerHalfSize,
					CreateIntersectionState.PreviewLocation + AxisX * MarkerHalfSize + AxisY * MarkerHalfSize,
					FColor::Red, SDPG_World, 4.0f);
				PDI->DrawLine(
					CreateIntersectionState.PreviewLocation - AxisX * MarkerHalfSize + AxisY * MarkerHalfSize,
					CreateIntersectionState.PreviewLocation + AxisX * MarkerHalfSize - AxisY * MarkerHalfSize,
					FColor::Red, SDPG_World, 4.0f);
			}

			IndicatorRadius = BuildSettings.DragEndpointAutoIntersectionRange;
			IndicatorInnerRadius = BuildSettings.SnapAutoIntersectionToClosestPointTolerance;
		}

		if (bIsAutoConnecting)
		{
			for (int32 Index = 0; Index < AutoConnectState.DestShapeConnectorInfos.Num(); Index++)
			{
				const bool bIsClosest = (Index == AutoConnectState.ClosestShapeConnectorInfoIndex);
				
				// Draw a square at the potential snap position
				const ZoneShapeConnectorRenderInfo& Info = AutoConnectState.DestShapeConnectorInfos[Index];
				const FColor& ChevronColor = bIsClosest ? FColor::Red : FColor::Silver;
				const FVector AxisX = Info.Foward.RotateAngleAxis(-45, Info.Up);
				const FVector AxisY = Info.Foward.RotateAngleAxis(45, Info.Up);
				DrawRectangle(PDI, Info.Position, AxisX, AxisY, ChevronColor, 20.f, 20.f, SDPG_World, 4.f);
			}
			IndicatorRadius = BuildSettings.DragEndpointAutoConnectRange;
		}

		// Draw auto connection/intersection range indicator
		if (IndicatorRadius > 0.0)
		{
			if (BuildSettings.bShow3DRadiusForAutoConnectionAndIntersection)
			{
				DrawWireSphere(PDI, Transform, IndicatorColor, IndicatorRadius, 32, SDPG_World, 0.0f, 0.001f, false);
			}
			else
			{
				DrawCircle(PDI, Center, FVector::XAxisVector, FVector::YAxisVector, IndicatorColor, IndicatorRadius, 32, SDPG_World);
			}
		}
		
		if (IndicatorInnerRadius > 0.0)
		{
			if (BuildSettings.bShow3DRadiusForAutoConnectionAndIntersection)
			{
				DrawWireSphere(PDI, Transform, InnerIndicatorColor, IndicatorInnerRadius, 24, SDPG_World, 0.0f, 0.001f, false);
			}
			else
			{
				DrawCircle(PDI, Center, FVector::XAxisVector, FVector::YAxisVector, InnerIndicatorColor, IndicatorInnerRadius, 24, SDPG_World);
			}
		}
	}

	PDI->SetHitProxy(nullptr);
}

void FZoneShapeComponentVisualizer::DrawVisualizationHUD(const UActorComponent* Component, const FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
{
	const UZoneShapeComponent* ShapeComp = Cast<const UZoneShapeComponent>(Component);
	{
		if (ShapeComp != nullptr && ShapeComp == GetEditedComponent())
		{
			check(SelectionState)
			int32 SelectedControlPoint = SelectionState->GetSelectedControlPoint();
			int32 LastPointIndexSelected = SelectionState->GetLastPointIndexSelected();
			if (SelectionState->GetSelectedPoints().Num() == 1 &&
				(LastPointIndexSelected == 0 || LastPointIndexSelected == (ShapeComp->GetNumPoints() - 1)))
			{
				const FIntRect CanvasRect = Canvas->GetViewRect();

				static const FText AutoConnectionHelp = LOCTEXT("ZoneShapeAutoConnectionMessage", "Auto Zone Shape Connection: Hold C and drag zone shape end point close to another shape connector to connect.");
				static const FText AutoIntersectionHelp = LOCTEXT("ZoneShapeAutoIntersectionMessage", "Auto Zone Shape Intersection: Hold X and drag zone shape end point close to another shape to create an intersection.");

				auto DisplaySnapToActorHelpText = [&](const FText& SnapHelpText, double YOffset)
				{
					int32 XL;
					int32 YL;
					StringSize(GEngine->GetLargeFont(), XL, YL, *SnapHelpText.ToString());
					const double DrawPositionX = FMath::FloorToDouble(CanvasRect.Min.X + (CanvasRect.Width() - XL) * 0.5);
					const double DrawPositionY = CanvasRect.Min.Y + 50.0 + YOffset;
					Canvas->DrawShadowedString(DrawPositionX, DrawPositionY, *SnapHelpText.ToString(), GEngine->GetLargeFont(), FLinearColor::Yellow);
				};
				if (CanAutoConnect(ShapeComp))
				{
					DisplaySnapToActorHelpText(AutoConnectionHelp, 0.0);
				}
				if (CanAutoCreateIntersection(ShapeComp))
				{
					DisplaySnapToActorHelpText(AutoIntersectionHelp, 20.0);
				}
			}
		}
	}
}

void FZoneShapeComponentVisualizer::ChangeSelectionState(int32 Index, bool bIsCtrlHeld) const
{
	check(SelectionState);
	SelectionState->Modify();

	TSet<int32>& SelectedPoints = SelectionState->ModifySelectedPoints();
	if (Index == INDEX_NONE)
	{
		SelectedPoints.Empty();
		SelectionState->SetLastPointIndexSelected(INDEX_NONE);
	}
	else if (!bIsCtrlHeld)
	{
		SelectedPoints.Empty();
		SelectedPoints.Add(Index);
		SelectionState->SetLastPointIndexSelected(Index);
	}
	else
	{
		// Add or remove from selection if Ctrl is held
		if (SelectedPoints.Contains(Index))
		{
			// If already in selection, toggle it off
			SelectedPoints.Remove(Index);

			if (SelectionState->GetLastPointIndexSelected() == Index)
			{
				if (SelectedPoints.Num() == 0)
				{
					// Last key selected: clear last key index selected
					SelectionState->SetLastPointIndexSelected(INDEX_NONE);
				}
				else
				{
					// Arbitrarily set last key index selected to first member of the set (so that it is valid)
					SelectionState->SetLastPointIndexSelected(*SelectedPoints.CreateConstIterator());
				}
			}
		}
		else
		{
			// Add to selection
			SelectedPoints.Add(Index);
			SelectionState->SetLastPointIndexSelected(Index);
		}
	}
}

const UZoneShapeComponent* FZoneShapeComponentVisualizer::UpdateSelectedShapeComponent(const HComponentVisProxy* VisProxy)
{
	check(SelectionState);
	const UZoneShapeComponent* NewShapeComp = CastChecked<const UZoneShapeComponent>(VisProxy->Component.Get());
	check(NewShapeComp);

	AActor* OldShapeOwningActor = SelectionState->GetShapePropertyPath().GetParentOwningActor();
	UZoneShapeComponent* OldShapeComp = GetEditedShapeComponent();

	const FComponentPropertyPath NewShapePropertyPath(NewShapeComp);
	SelectionState->SetShapePropertyPath(NewShapePropertyPath);

	AActor* NewShapeOwningActor = NewShapePropertyPath.GetParentOwningActor();

	if (NewShapePropertyPath.IsValid())
	{
		if (OldShapeOwningActor != NewShapeOwningActor ||  OldShapeComp != NewShapeComp)
		{
			// Reset selection state if we are selecting a different actor to the one previously selected
			ChangeSelectionState(INDEX_NONE, false);
			SelectionState->SetSelectedSegmentIndex(INDEX_NONE);
			SelectionState->SetSelectedControlPoint(INDEX_NONE);
			SelectionState->SetSelectedControlPointType(FZoneShapeControlPointType::None);
		}

		if (OldShapeComp != NewShapeComp)
		{
			bIsSelectingComponent = true; // Prevent the selection from clearing our own selection state.
        	GEditor->SelectNone(/*bNoteSelectionChange*/true, /*bDeselectBSPSurfs*/true);
        	GEditor->SelectActor(NewShapeOwningActor, /*bInSelected*/false, /*bNotify*/true);
			GEditor->SelectComponent(const_cast<UZoneShapeComponent*>(NewShapeComp), /*bInSelected*/true, /*bNotify*/true);
			bIsSelectingComponent = false;
		}
		
		return NewShapeComp;
	}
	SelectionState->SetShapePropertyPath(FComponentPropertyPath());

	return nullptr;
}


bool FZoneShapeComponentVisualizer::GetLastSelectedPointRotation(FQuat& OutRotation) const
{
	bool bResult = false;
	if (const UZoneShapeComponent* ShapeComp = GetEditedShapeComponent())
	{
		check(SelectionState);
		const TConstArrayView<FZoneShapePoint> ShapePoints = ShapeComp->GetPoints();
		const int32 LastPointIndexSelected = SelectionState->GetLastPointIndexSelected();
		if (ShapePoints.IsValidIndex(LastPointIndexSelected))
		{
			check(SelectionState->GetSelectedPoints().Contains(LastPointIndexSelected));
			OutRotation = ShapeComp->GetComponentTransform().GetRotation() * ShapePoints[LastPointIndexSelected].Rotation.Quaternion();
			bResult = true;
		}
	}
	return bResult;
}

bool FZoneShapeComponentVisualizer::VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* VisProxy, const FViewportClick& Click)
{
	if (VisProxy && VisProxy->Component.IsValid())
	{
		if (VisProxy->IsA(HZoneShapePointProxy::StaticGetType()))
		{
			// Control point clicked
			const FScopedTransaction Transaction(LOCTEXT("SelectShapePoint", "Select Shape Point"));

			SelectionState->Modify();

			if (UpdateSelectedShapeComponent(VisProxy))
			{
				const HZoneShapePointProxy* PointProxy = static_cast<HZoneShapePointProxy*>(VisProxy);
				// Modify the selection state, unless right-clicking on an already selected key
				const TSet<int32>& SelectedPoints = SelectionState->GetSelectedPoints();
				if (Click.GetKey() != EKeys::RightMouseButton || !SelectedPoints.Contains(PointProxy->PointIndex))
				{
					ChangeSelectionState(PointProxy->PointIndex, InViewportClient->IsCtrlPressed());
				}
				SelectionState->SetSelectedSegmentIndex(INDEX_NONE);
				SelectionState->SetSelectedControlPoint(INDEX_NONE);
				SelectionState->SetSelectedControlPointType(FZoneShapeControlPointType::None);

				if (SelectionState->GetLastPointIndexSelected() == INDEX_NONE)
				{
					SelectionState->SetShapePropertyPath(FComponentPropertyPath());
					return false;
				}

				return true;
			}
		}
		else if (VisProxy->IsA(HZoneShapeSegmentProxy::StaticGetType()))
		{
			// Shape segment clicked
			const FScopedTransaction Transaction(LOCTEXT("SelectShapeSegment", "Select Shape Segment"));
			SelectionState->Modify();

			if (const UZoneShapeComponent* ShapeComp = UpdateSelectedShapeComponent(VisProxy))
			{
				const FTransform& LocalToWorld = ShapeComp->GetComponentTransform();
				const HZoneShapeSegmentProxy* SegmentProxy = static_cast<HZoneShapeSegmentProxy*>(VisProxy);

				// Find nearest point on shape.
				ChangeSelectionState(INDEX_NONE, false);
				SelectionState->SetSelectedSegmentIndex(SegmentProxy->SegmentIndex);
				SelectionState->SetSelectedControlPoint(INDEX_NONE);
				SelectionState->SetSelectedControlPointType(FZoneShapeControlPointType::None);

				const int32 NumPoints = ShapeComp->GetNumPoints();
				const int32 StartIndex = SegmentProxy->SegmentIndex;
				const int32 EndIndex = (SegmentProxy->SegmentIndex + 1) % NumPoints;

				const TConstArrayView<FZoneShapePoint> ShapePoints = ShapeComp->GetPoints();

				FVector StartPosition(0), StartControlPoint(0), EndControlPoint(0), EndPosition(0);
				UE::ZoneShape::Utilities::GetCubicBezierPointsFromShapeSegment(ShapePoints[StartIndex], ShapePoints[EndIndex], LocalToWorld.ToMatrixWithScale(), StartPosition, StartControlPoint, EndControlPoint, EndPosition);

				const FVector RaySegStart = Click.GetOrigin();
				const FVector RaySegEnd = Click.GetOrigin() + Click.GetDirection() * 50000.0f;

				FVector ClosestPoint;
				float ClosestT = 0.0f;

				UE::CubicBezier::SegmentClosestPointApproximate(RaySegStart, RaySegEnd, StartPosition, StartControlPoint, EndControlPoint, EndPosition, ClosestPoint, ClosestT);

				SelectionState->SetSelectedSegmentPoint(ClosestPoint);
				SelectionState->SetSelectedSegmentT(ClosestT);

				return true;
			}
		}
		else if (VisProxy->IsA(HZoneShapeControlPointProxy::StaticGetType()))
		{
			// Shape segment clicked
			const FScopedTransaction Transaction(LOCTEXT("SelectShapeSegment", "Select Shape Segment"));
			SelectionState->Modify();

			if (UpdateSelectedShapeComponent(VisProxy))
			{
				// Tangent handle clicked
				const HZoneShapeControlPointProxy* ControlPointProxy = static_cast<HZoneShapeControlPointProxy*>(VisProxy);

				// Note: don't change key selection when a tangent handle is clicked
				SelectionState->SetSelectedSegmentIndex(INDEX_NONE);
				SelectionState->SetSelectedControlPoint(ControlPointProxy->PointIndex);
				SelectionState->SetSelectedControlPointType(ControlPointProxy->bInControlPoint ? FZoneShapeControlPointType::In : FZoneShapeControlPointType::Out);

				return true;
			}
		}
		else if (VisProxy->IsA(HZoneShapeVisProxy::StaticGetType()))
		{
			// Control point clicked
			const FScopedTransaction Transaction(LOCTEXT("SelectShape", "Select Shape"));

			SelectionState->Modify();

			if (UpdateSelectedShapeComponent(VisProxy))
			{
				ChangeSelectionState(INDEX_NONE, false);
				SelectionState->SetSelectedSegmentIndex(INDEX_NONE);
				SelectionState->SetSelectedControlPoint(INDEX_NONE);
				SelectionState->SetSelectedControlPointType(FZoneShapeControlPointType::None);

				return true;
			}
		}
	}

	return false;
}


UZoneShapeComponent* FZoneShapeComponentVisualizer::GetEditedShapeComponent() const
{
	check(SelectionState);
	return Cast<UZoneShapeComponent>(SelectionState->GetShapePropertyPath().GetComponent());
}

UActorComponent* FZoneShapeComponentVisualizer::GetEditedComponent() const
{
	return Cast<UActorComponent>(GetEditedShapeComponent());
}

bool FZoneShapeComponentVisualizer::GetWidgetLocation(const FEditorViewportClient* ViewportClient, FVector& OutLocation) const
{
	if (const UZoneShapeComponent* ShapeComp = GetEditedShapeComponent())
	{
		check(SelectionState);
		const TConstArrayView<FZoneShapePoint> ShapePoints = ShapeComp->GetPoints();

		if (SelectionState->GetSelectedControlPoint() != INDEX_NONE)
		{
			// If control point index is set, use that
			if (bControlPointPositionCaptured)
			{
				OutLocation = ShapeComp->GetComponentTransform().TransformPosition(ControlPointPosition);
			}
			else
			{
				check(SelectionState->GetSelectedControlPoint() < ShapePoints.Num());
				const FZoneShapePoint& Point = ShapePoints[SelectionState->GetSelectedControlPoint()];
				if (SelectionState->GetSelectedControlPointType() == FZoneShapeControlPointType::Out)
				{
					OutLocation = ShapeComp->GetComponentTransform().TransformPosition(Point.GetOutControlPoint());
				}
				else
				{
					OutLocation = ShapeComp->GetComponentTransform().TransformPosition(Point.GetInControlPoint());
				}
			}

			return true;
		}
		else if (SelectionState->GetSelectedSegmentIndex() != INDEX_NONE)
		{
			return false;
		}
		else if (SelectionState->GetLastPointIndexSelected() != INDEX_NONE)
		{
			// Otherwise use the last key index set
			const int32 LastPointIndexSelected = SelectionState->GetLastPointIndexSelected();
			check(LastPointIndexSelected >= 0);
			if (LastPointIndexSelected < ShapePoints.Num())
			{
				check(SelectionState->GetSelectedPoints().Contains(LastPointIndexSelected));
				const FZoneShapePoint& Point = ShapePoints[LastPointIndexSelected];
				OutLocation = ShapeComp->GetComponentTransform().TransformPosition(Point.Position);
				OutLocation += DuplicateAccumulatedDrag;
				return true;
			}
		}
	}

	return false;
}

bool FZoneShapeComponentVisualizer::GetCustomInputCoordinateSystem(const FEditorViewportClient* ViewportClient, FMatrix& OutMatrix) const
{
	bool bResult = false;
	if (bHasCachedRotation)
	{
		OutMatrix = FRotationMatrix::Make(CachedRotation);
		bResult = true;
	}
	else
	{
		if (ViewportClient->GetWidgetCoordSystemSpace() == COORD_Local || ViewportClient->GetWidgetMode() == UE::Widget::WM_Rotate)
		{
			FQuat Rotation = FQuat::Identity;
			if (GetLastSelectedPointRotation(Rotation))
			{
				OutMatrix = FRotationMatrix::Make(Rotation);
				bResult = true;
			}
		}
	}

	return bResult;
}

bool FZoneShapeComponentVisualizer::IsVisualizingArchetype() const
{
	UZoneShapeComponent* ShapeComp = GetEditedShapeComponent();
	return (ShapeComp && ShapeComp->GetOwner() && FActorEditorUtils::IsAPreviewOrInactiveActor(ShapeComp->GetOwner()));
}


bool FZoneShapeComponentVisualizer::IsAnySelectedPointIndexOutOfRange(const UZoneShapeComponent& Comp) const
{
	check(SelectionState);
	const TSet<int32>& SelectedPoints = SelectionState->GetSelectedPoints();
	const int32 NumPoints = Comp.GetNumPoints();
	return Algo::AnyOf(SelectedPoints, [NumPoints](int32 Index) { return Index >= NumPoints; });
}

bool FZoneShapeComponentVisualizer::IsSinglePointSelected() const
{
	UZoneShapeComponent* ShapeComp = GetEditedShapeComponent();
	check(SelectionState);
	const TSet<int32>& SelectedPoints = SelectionState->GetSelectedPoints();
	return (ShapeComp != nullptr &&
		SelectedPoints.Num() == 1 &&
		SelectionState->GetLastPointIndexSelected() != INDEX_NONE);
}

bool FZoneShapeComponentVisualizer::HandleInputDelta(FEditorViewportClient* ViewportClient, FViewport* Viewport, FVector& DeltaTranslate, FRotator& DeltaRotate, FVector& DeltaScale)
{
	if (const UZoneShapeComponent* ShapeComp = GetEditedShapeComponent())
	{
		check(SelectionState);

		if (IsAnySelectedPointIndexOutOfRange(*ShapeComp))
		{
			// Something external has changed the number of shape points, meaning that the cached selected keys are no longer valid
			EndEditing();
			return false;
		}

		const int32 LastPointIndexSelected = SelectionState->GetLastPointIndexSelected();
		if (SelectionState->GetSelectedControlPoint() != INDEX_NONE)
		{
			return TransformSelectedControlPoint(DeltaTranslate);
		}
		else if (SelectionState->GetSelectedPoints().Num() > 0)
		{
			if (!ViewportClient->IsAltPressed() &&
				SelectionState->GetSelectedPoints().Num() == 1 &&
				(LastPointIndexSelected == 0 || LastPointIndexSelected == (ShapeComp->GetNumPoints() - 1)))
			{
				// Cache the selected index
				SelectedPointForConnecting = LastPointIndexSelected;
				const FZoneShapePoint& DraggedPoint = ShapeComp->GetPoints()[SelectedPointForConnecting];

#if WITH_EDITOR
				if (ViewportClient->Viewport->KeyState(EKeys::C))
				{
					DetectCloseByShapeForAutoConnection(ShapeComp, DraggedPoint);
				}
				else if (ViewportClient->Viewport->KeyState(EKeys::X) && CanAutoCreateIntersection(ShapeComp))
				{
					DetectCloseByShapeForAutoIntersectionCreation(ShapeComp, DraggedPoint);
				}
#endif
			}

			if (ViewportClient->IsAltPressed())
			{
				if (ViewportClient->GetWidgetMode() == UE::Widget::WM_Translate && ViewportClient->GetCurrentWidgetAxis() != EAxisList::None)
				{
					if (bAllowDuplication)
					{
						static const float DuplicationDeadZoneSqr = FMath::Square(10.0f);

						DuplicateAccumulatedDrag += DeltaTranslate;
						if (DuplicateAccumulatedDrag.SizeSquared() >= DuplicationDeadZoneSqr)
						{
							DuplicatePointForAltDrag(DuplicateAccumulatedDrag);
							DuplicateAccumulatedDrag = FVector::ZeroVector;
							bAllowDuplication = false;
						}

						return true;
					}
					else
					{
						return TransformSelectedPoints(ViewportClient, DeltaTranslate, DeltaRotate, DeltaScale);
					}
				}
			}
			else
			{
				return TransformSelectedPoints(ViewportClient, DeltaTranslate, DeltaRotate, DeltaScale);
			}
		}
	}

	return false;
}

void FZoneShapeComponentVisualizer::DetectCloseByShapeForAutoConnection(const UZoneShapeComponent* ShapeComp, const FZoneShapePoint& DraggedPoint)
{
	ClearAutoConnectingStatus();
	bIsAutoConnecting = true;

	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
	if (!ZoneGraphSettings)
	{
		return;
	}
	UZoneGraphSubsystem* ZoneGraph = UWorld::GetSubsystem<UZoneGraphSubsystem>(ShapeComp->GetWorld());
	if (!ZoneGraph)
	{
		return;
	}

	const FZoneShapeConnector* SourceConnector = ShapeComp->GetShapeConnectorByPointIndex(SelectedPointForConnecting);
	if (!SourceConnector)
	{
		return;
	}

	const FZoneGraphBuildSettings& BuildSettings = ZoneGraphSettings->GetBuildSettings();
	const TArray<FZoneGraphBuilderRegisteredComponent>& RegisteredShapeComponents = ZoneGraph->GetBuilder().GetRegisteredZoneShapeComponents();

	const FTransform& SourceTransform = ShapeComp->GetComponentTransform();
	const FVector SourceWorldPosition = SourceTransform.TransformPosition(SourceConnector->Position);
	const FVector DraggedPointWorldPosition = SourceTransform.TransformPosition(DraggedPoint.Position);
	
	TArray<uint32> QueryResults;
	const FBox Bounds = FBox::BuildAABB(DraggedPointWorldPosition, FVector(BuildSettings.DragEndpointAutoConnectRange));
	ZoneGraph->GetBuilder().QueryHashGrid(Bounds, QueryResults);
		
	double ShortestDistance = BuildSettings.DragEndpointAutoConnectRange;
	for (const uint32 ComponentIndex : QueryResults)
	{
		check(RegisteredShapeComponents.IsValidIndex(int32(ComponentIndex)));
		const UZoneShapeComponent* DestShapeComp = RegisteredShapeComponents[ComponentIndex].Component;
		if (!DestShapeComp
			|| DestShapeComp == ShapeComp
			|| ShapeComp->GetComponentLevel() != DestShapeComp->GetComponentLevel())
		{
			continue;
		}

		const FTransform& DestTransform = DestShapeComp->GetComponentTransform();
		TConstArrayView<FZoneShapeConnector> DestConnectors = DestShapeComp->GetShapeConnectors();
		TConstArrayView<FZoneShapeConnection> DestConnections = DestShapeComp->GetConnectedShapes();

		for (int32 ConIndex = 0; ConIndex < DestConnectors.Num(); ConIndex++)
		{
			const FZoneShapeConnector& DestConnector = DestConnectors[ConIndex];
			if (SourceConnector == &DestConnector
				|| SourceConnector->LaneProfile != DestConnector.LaneProfile)
			{
				continue;;
			}

			const bool bOccupied = ConIndex < DestConnections.Num() && DestConnections[ConIndex].ShapeComponent.IsValid();
			if (bOccupied)
			{
				continue;
			}

			// Check that the profile orientation matches before connecting.
			const FZoneLaneProfile* LaneProfile = ZoneGraphSettings->GetLaneProfileByRef(SourceConnector->LaneProfile);
			if (LaneProfile
				&& (LaneProfile->IsSymmetrical() || SourceConnector->bReverseLaneProfile != DestConnector.bReverseLaneProfile))
			{
				const FVector DestWorldPosition = DestTransform.TransformPosition(DestConnector.Position);
				const double Distance = FVector::Dist(SourceWorldPosition, DestWorldPosition);

				if (Distance < BuildSettings.DragEndpointAutoConnectRange)
				{
					const FVector DestWorldNormal = DestTransform.TransformVector(DestConnector.Normal);
					const FVector DestWorldUp = DestTransform.TransformVector(DestConnector.Up);
					const int32 InfoIndex = AutoConnectState.DestShapeConnectorInfos.Add({ DestWorldPosition, DestWorldNormal, DestWorldUp });

					if (Distance < ShortestDistance)
					{
						ShortestDistance = Distance;
						AutoConnectState.ClosestShapeConnectorInfoIndex = InfoIndex;
						AutoConnectState.NearestPointWorldPosition = DestWorldPosition;
						AutoConnectState.NearestPointWorldNormal = DestWorldNormal;
					}
				}

			}
		}
	}
}

void FZoneShapeComponentVisualizer::DetectCloseByShapeForAutoIntersectionCreation(const UZoneShapeComponent* ShapeComp, const FZoneShapePoint& DraggedPoint)
{
	ClearAutoIntersectionStatus();
	bIsCreatingIntersection = true;

	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
	if (!ZoneGraphSettings)
	{
		return;
	}
	UZoneGraphSubsystem* ZoneGraph = UWorld::GetSubsystem<UZoneGraphSubsystem>(ShapeComp->GetWorld());
	if (!ZoneGraph)
	{
		return;
	}

	const FZoneGraphBuildSettings& BuildSettings = ZoneGraphSettings->GetBuildSettings();
	const TArray<FZoneGraphBuilderRegisteredComponent>& RegisteredShapeComponents = ZoneGraph->GetBuilder().GetRegisteredZoneShapeComponents();

	const FTransform& SourceTransform = ShapeComp->GetComponentTransform();
	FVector DraggedPointWorldPosition = SourceTransform.TransformPosition(DraggedPoint.Position);

	TArray<uint32> QueryResults;
	const FBox Bounds = FBox::BuildAABB(DraggedPointWorldPosition, FVector(BuildSettings.DragEndpointAutoIntersectionRange));
	ZoneGraph->GetBuilder().QueryHashGrid(Bounds, QueryResults);
	
	double ClosestDistanceToSegment = std::numeric_limits<double>::infinity();
	for (uint32 ComponentIndex : QueryResults)
	{
		check(RegisteredShapeComponents.IsValidIndex(int32(ComponentIndex)));
		UZoneShapeComponent* DestShapeComp = RegisteredShapeComponents[ComponentIndex].Component;
		if (!DestShapeComp ||
			DestShapeComp == ShapeComp ||
			ShapeComp->GetComponentLevel() != DestShapeComp->GetComponentLevel())
		{
			continue;
		}

		const FTransform& DestTransform = DestShapeComp->GetComponentTransform();
		TConstArrayView<FZoneShapePoint> DestPoints = DestShapeComp->GetPoints();

		FVector DraggedPointRelativePosition = DestTransform.InverseTransformPosition(DraggedPointWorldPosition);

		if (DestShapeComp->GetShapeType() == FZoneShapeType::Spline)
		{
			// Spline
			const FZoneLaneProfile* LaneProfile = ZoneGraphSettings->GetLaneProfileByRef(DestShapeComp->GetCommonLaneProfile());
			const double HalfLanesTotalWidth = LaneProfile ? LaneProfile->GetLanesTotalWidth() * 0.5 : 0.0;

			// Find closest point to the stem of the spline.
			for (int32 Index = 0; Index < DestPoints.Num() - 1; Index++)
			{
				const FZoneShapePoint& CurrPoint = DestPoints[Index];
				const FZoneShapePoint& NextPoint = DestPoints[Index + 1];
				
				FVector ClosestPoint;
				float ClosestT = 0.0f;
				UE::CubicBezier::ClosestPointApproximate(
					DraggedPointRelativePosition,
					CurrPoint.Position,
					CurrPoint.GetOutControlPoint(),
					NextPoint.Position,
					NextPoint.GetInControlPoint(),
					ClosestPoint,
					ClosestT);

				const double Dist = FVector::Dist(DraggedPointRelativePosition, ClosestPoint);
				if (Dist < (BuildSettings.DragEndpointAutoIntersectionRange + HalfLanesTotalWidth)
					&& Dist < ClosestDistanceToSegment)
				{
					ClosestDistanceToSegment = Dist;
					CreateIntersectionState.WeakTargetShapeComponent = DestShapeComp;
					CreateIntersectionState.OverlappingSegmentIndex = Index;
					CreateIntersectionState.OverlappingSegmentT = ClosestT;
					CreateIntersectionState.PreviewLocation = DestTransform.TransformPosition(ClosestPoint);
				}
			}
		}
		else
		{
			// Polygon
			// Polygon defines the outline of the polygon, to make the behavior comparable to the spline case,
			// just use linear segments between the lane profile points.
			TArray<FZoneLaneProfile> PolyLaneProfiles;
			DestShapeComp->GetPolygonLaneProfiles(PolyLaneProfiles);
			check(DestPoints.Num() == PolyLaneProfiles.Num());

			int32 PrevLaneProfilePointIndex = INDEX_NONE;
			if (!DestPoints.IsEmpty() && DestPoints.Last().Type == FZoneShapePointType::LaneProfile)
			{
				PrevLaneProfilePointIndex = DestPoints.Num() - 1;
			}
			
			for (int32 Index = 0; Index < DestPoints.Num(); Index++)
			{
				const FZoneShapePoint& CurrPoint = DestPoints[Index];
				if (CurrPoint.Type == FZoneShapePointType::LaneProfile)
				{
					if (PrevLaneProfilePointIndex != INDEX_NONE)
					{
						const FZoneShapePoint& PrevPoint = DestPoints[PrevLaneProfilePointIndex];
						const FVector ClosestPoint = FMath::ClosestPointOnSegment(DraggedPointRelativePosition, PrevPoint.Position, CurrPoint.Position);

						const double PrevHalfLanesTotalWidth = PolyLaneProfiles[PrevLaneProfilePointIndex].GetLanesTotalWidth();
						const double CurrHalfLanesTotalWidth = PolyLaneProfiles[Index].GetLanesTotalWidth();
						const double HalfLanesTotalWidth = FMath::Min(PrevHalfLanesTotalWidth, CurrHalfLanesTotalWidth) * 0.5;
						
						const double Dist = FVector::Dist(DraggedPointRelativePosition, ClosestPoint);
						if (Dist < (BuildSettings.DragEndpointAutoIntersectionRange + HalfLanesTotalWidth)
							&& Dist < ClosestDistanceToSegment)
						{
							ClosestDistanceToSegment = Dist;
							CreateIntersectionState.WeakTargetShapeComponent = DestShapeComp;
							CreateIntersectionState.OverlappingSegmentIndex = -1; // Not used for polygons
							CreateIntersectionState.OverlappingSegmentT = 0.0; // Not used for polygons
							CreateIntersectionState.PreviewLocation = DestTransform.TransformPosition(ClosestPoint);
						}
					}
					PrevLaneProfilePointIndex = Index;
				}
			}
		}
	}

	// If the dragged point is close to a point on spline, or un-connected lane point in polygon, try to snap to that. 
	if (UZoneShapeComponent* TargetShapeComponent = CreateIntersectionState.WeakTargetShapeComponent.Get())
	{
		const FTransform& TargetShapeCompTransform = TargetShapeComponent->GetComponentTransform();
		CreateIntersectionState.ClosePointIndex = INDEX_NONE;
		
		TArray<FZoneShapePoint>& TargetShapePoints = TargetShapeComponent->GetMutablePoints();
		const int32 NumPoints = TargetShapePoints.Num();

		TConstArrayView<FZoneShapeConnector> DestConnectors = TargetShapeComponent->GetShapeConnectors();
		TConstArrayView<FZoneShapeConnection> DestConnections = TargetShapeComponent->GetConnectedShapes();
		
		static const double SnapToleranceSqr = FMath::Square(BuildSettings.SnapAutoIntersectionToClosestPointTolerance);
		double ShortestDistanceSqr = SnapToleranceSqr;
		
		for (int32 PointIndex = 0; PointIndex < NumPoints; PointIndex++)
		{
			const FZoneShapePoint& CurrTargetPoint = TargetShapePoints[PointIndex];
			
			// Only allow to snap to lane profile points on polygons.
			if (TargetShapeComponent->GetShapeType() == FZoneShapeType::Polygon
				&& CurrTargetPoint.Type != FZoneShapePointType::LaneProfile)
			{
				continue;
			}

			// Prevent snapping to already connected points.
			bool bOccupied = false;
			for (int ConIndex = 0; ConIndex < DestConnectors.Num(); ConIndex++)
			{
				if (DestConnectors[ConIndex].PointIndex == PointIndex)
				{
					bOccupied = ConIndex < DestConnections.Num() && DestConnections[ConIndex].ShapeComponent.IsValid();
					if (bOccupied)
					{
						break;
					}
				}
			}
			if (bOccupied)
			{
				continue;
			}

			const FVector TargetPointWorldPosition = TargetShapeCompTransform.TransformPosition(CurrTargetPoint.Position);
			const double DistSqr = FVector::DistSquared(DraggedPointWorldPosition, TargetPointWorldPosition);
			
			if (DistSqr < SnapToleranceSqr
				&& DistSqr < ShortestDistanceSqr)
			{
				ShortestDistanceSqr = DistSqr;
				CreateIntersectionState.ClosePointIndex = PointIndex;
				CreateIntersectionState.PreviewLocation = TargetPointWorldPosition;
			}
		}
	}
}

bool FZoneShapeComponentVisualizer::TransformSelectedControlPoint(const FVector& DeltaTranslate)
{
	if (UZoneShapeComponent* ShapeComp = GetEditedShapeComponent())
	{
		check(SelectionState);
		check(SelectionState->GetSelectedControlPoint() != INDEX_NONE);

		TArray<FZoneShapePoint>& ShapePoints = ShapeComp->GetMutablePoints();
		const int32 NumPoints = ShapePoints.Num();
		check(SelectionState->GetSelectedControlPoint() < NumPoints);

		if (!DeltaTranslate.IsZero())
		{
			ShapeComp->Modify();

			if (!bControlPointPositionCaptured)
			{
				// We capture the control point position on first update and use that as the gizmo position.
				// That allows us to constrain the handle locations as needed, and have the gizmo follow the user input.
				bControlPointPositionCaptured = true;

				const FZoneShapePoint& EditedPoint = ShapePoints[SelectionState->GetSelectedControlPoint()];
				if (EditedPoint.Type == FZoneShapePointType::Bezier || EditedPoint.Type == FZoneShapePointType::LaneProfile)
				{
					if (SelectionState->GetSelectedControlPointType() == FZoneShapeControlPointType::Out)
					{
						ControlPointPosition = EditedPoint.GetOutControlPoint();
					}
					else
					{
						ControlPointPosition = EditedPoint.GetInControlPoint();
					}
				}
			}

			ControlPointPosition += ShapeComp->GetComponentTransform().InverseTransformVector(DeltaTranslate);

			FZoneShapePoint& EditedPoint = ShapePoints[SelectionState->GetSelectedControlPoint()];

			if (EditedPoint.Type == FZoneShapePointType::Bezier || EditedPoint.Type == FZoneShapePointType::LaneProfile)
			{
				// Note: Lane control points will get adjusted to fit the lane profile in UpdateShape() below.
				if (SelectionState->GetSelectedControlPointType() == FZoneShapeControlPointType::Out)
				{
					EditedPoint.SetOutControlPoint(ControlPointPosition);
				}
				else
				{
					EditedPoint.SetInControlPoint(ControlPointPosition);
				}
			}
		}

		ShapeComp->UpdateShape();
		NotifyPropertyModified(ShapeComp, ShapePointsProperty);

		return true;
	}

	return false;
}

bool FZoneShapeComponentVisualizer::TransformSelectedPoints(const FEditorViewportClient* ViewportClient, const FVector& DeltaTranslate, const FRotator& DeltaRotate, const FVector& DeltaScale) const
{
	if (UZoneShapeComponent* ShapeComp = GetEditedShapeComponent())
	{
		check(SelectionState);
		TArray<FZoneShapePoint>& ShapePoints = ShapeComp->GetMutablePoints();
		const int32 NumPoints = ShapePoints.Num();
		check(SelectionState->GetLastPointIndexSelected() != INDEX_NONE);
		check(SelectionState->GetLastPointIndexSelected() >= 0);
		check(SelectionState->GetLastPointIndexSelected() < NumPoints);
		const TSet<int32>& SelectedPoints = SelectionState->GetSelectedPoints();
		const int32 LastPointIndexSelected = SelectionState->GetLastPointIndexSelected();
		check(SelectedPoints.Num() > 0);
		check(SelectedPoints.Contains(LastPointIndexSelected));

		ShapeComp->Modify();

		for (const int32 SelectedIndex : SelectedPoints)
		{
			check(SelectedIndex >= 0);
			check(SelectedIndex < NumPoints);

			FZoneShapePoint& EditedPoint = ShapePoints[SelectedIndex];

			if (!DeltaTranslate.IsZero())
			{
				const FVector LocalDelta = ShapeComp->GetComponentTransform().InverseTransformVector(DeltaTranslate);
				EditedPoint.Position += LocalDelta;
			}

			if (!DeltaRotate.IsZero())
			{
				FQuat NewRot = ShapeComp->GetComponentTransform().GetRotation() * EditedPoint.Rotation.Quaternion(); // convert local-space rotation to world-space
				NewRot = DeltaRotate.Quaternion() * NewRot; // apply world-space rotation
				NewRot = ShapeComp->GetComponentTransform().GetRotation().Inverse() * NewRot; // convert world-space rotation to local-space
				EditedPoint.Rotation = NewRot.Rotator();
			}

			if (DeltaScale.X != 0.0f)
			{
				if (EditedPoint.Type == FZoneShapePointType::Bezier)
				{
					EditedPoint.TangentLength *= (1.0f + DeltaScale.X);
				}
			}
		}

		ShapeComp->UpdateShape();
		NotifyPropertyModified(ShapeComp, ShapePointsProperty);
		GEditor->RedrawLevelEditingViewports(true);

		return true;
	}

	return false;
}

bool FZoneShapeComponentVisualizer::HandleInputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event)
{
	bool bHandled = false;

	UZoneShapeComponent* ShapeComp = GetEditedShapeComponent();
	if (!ShapeComp)
	{
		return false;
	}

	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
	if (!ZoneGraphSettings)
	{
		return false;
	}
	const FZoneGraphBuildSettings& BuildSettings = ZoneGraphSettings->GetBuildSettings();

	if (IsAnySelectedPointIndexOutOfRange(*ShapeComp))
	{
		// Something external has changed the number of shape points, meaning that the cached selected keys are no longer valid
		EndEditing();
		return false;
	}

	if (Key == EKeys::LeftMouseButton && Event == IE_Released)
	{
		// Reset duplication on LMB release
		bAllowDuplication = true;
		DuplicateAccumulatedDrag = FVector::ZeroVector;

		bControlPointPositionCaptured = false;
		ControlPointPosition = FVector::ZeroVector;

		bHasCachedRotation = false;
		CachedRotation = FQuat::Identity;

		TArray<FZoneShapePoint>& ShapePoints = ShapeComp->GetMutablePoints();
		if (ShapePoints.IsValidIndex(SelectedPointForConnecting))
		{
			if (bIsAutoConnecting)
			{
				const FZoneLaneProfile* LaneProfile = ZoneGraphSettings->GetLaneProfileByRef(ShapeComp->GetCommonLaneProfile());
				double HalfLanesTotalWidth = LaneProfile ? LaneProfile->GetLanesTotalWidth() * 0.5 : 0.0;

				FZoneShapePoint& DraggedPoint = ShapePoints[SelectedPointForConnecting];
				const FZoneShapeConnector* SourceConnector = ShapeComp->GetShapeConnectorByPointIndex(SelectedPointForConnecting);
				
				if (SourceConnector && AutoConnectState.ClosestShapeConnectorInfoIndex != INDEX_NONE)
				{
					const FTransform& SourceTransform = ShapeComp->GetComponentTransform();
					const FVector SourceWorldNormal = SourceTransform.TransformVector(SourceConnector->Normal);

					const double ConnectionSnapAngleCos = FMath::Cos(FMath::DegreesToRadians(BuildSettings.ConnectionSnapAngle));

					UE::ZoneGraph::Editor::Private::SnapConnect(
						ShapeComp,
						DraggedPoint,
						SourceTransform,
						SourceWorldNormal,
						AutoConnectState.NearestPointWorldPosition,
						AutoConnectState.NearestPointWorldNormal,
						ConnectionSnapAngleCos,
						HalfLanesTotalWidth);
				}
			}

			if (bIsCreatingIntersection)
			{
				CreateIntersection(ShapeComp);
			}
		}
		
		ClearAutoConnectingStatus();
		ClearAutoIntersectionStatus();
	}

	if (Key == EKeys::C && Event == IE_Released)
	{
		ClearAutoConnectingStatus();
	}

	if (Key == EKeys::X && Event == IE_Released)
	{
		ClearAutoIntersectionStatus();
	}

	if (Key == EKeys::LeftMouseButton && Event == IE_Pressed)
	{
		bHasCachedRotation = false;
		CachedRotation = FQuat::Identity;

		// Cache the widget rotation when mouse is pressed down to avoid feedback effects during gizmo interaction.
		if (ViewportClient->GetWidgetCoordSystemSpace() == COORD_Local || ViewportClient->GetWidgetMode() == UE::Widget::WM_Rotate)
		{
			bHasCachedRotation = GetLastSelectedPointRotation(CachedRotation);
		}
	}

	if (Event == IE_Pressed)
	{
		const int32 SelectedIndex = SelectionState->GetLastPointIndexSelected();

		// Add a new point to the shape when you hold the V key and press left mouse button
		if (ShapeComp && Key == EKeys::LeftMouseButton && Viewport->KeyState(EKeys::V) && SelectedIndex != INDEX_NONE)
		{
			// Get clicked position
			UWorld* World = ViewportClient->GetWorld();
			FSceneViewFamilyContext ViewFamily(FSceneViewFamilyContext::ConstructionValues(ViewportClient->Viewport, ViewportClient->GetScene(), ViewportClient->EngineShowFlags)
				.SetRealtimeUpdate(ViewportClient->IsRealtime()));
			FSceneView* View = ViewportClient->CalcSceneView(&ViewFamily);
			int32 MouseX = ViewportClient->Viewport->GetMouseX();
			int32 MouseY = ViewportClient->Viewport->GetMouseY();
			FViewportCursorLocation MouseViewportRay(View, ViewportClient, MouseX, MouseY);
			FVector MouseViewportRayDirection = MouseViewportRay.GetDirection();

			FVector Start = MouseViewportRay.GetOrigin();
			FVector End = Start + WORLD_MAX * MouseViewportRayDirection;
			if (ViewportClient->IsOrtho())
			{
				Start -= WORLD_MAX * MouseViewportRayDirection;
			}
			FHitResult Hit;
			FCollisionQueryParams QueryParams;
			QueryParams.bTraceComplex = true;
			if (World->LineTraceSingleByChannel(Hit, Start, End, ECollisionChannel::ECC_WorldStatic, QueryParams))
			{
				// Add a new point at the position
				const FScopedTransaction Transaction(LOCTEXT("AddShapePointAndSnap", "Add Shape Point And Snap To Floor"));
				AddSegment(Hit.Location, SelectedIndex, ShapeComp);
			}
			else
			{
				UE_LOGF(LogZoneShapeComponentVisualizer, Warning, "No hit found on click.");
			}
			return true;
		}

		bHandled = ShapeComponentVisualizerActions->ProcessCommandBindings(Key, FSlateApplication::Get().GetModifierKeys(), false);
	}

	return bHandled;
}

bool FZoneShapeComponentVisualizer::HandleBoxSelect(const FBox& InBox, FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	const FScopedTransaction Transaction(LOCTEXT("HandleBoxSelect", "Box Select Shape Points"));
	check(SelectionState);
	SelectionState->Modify();

	if (const UZoneShapeComponent* ShapeComp = GetEditedShapeComponent())
	{
		bool bSelectionChanged = false;

		const TConstArrayView<FZoneShapePoint> ShapePoints = ShapeComp->GetPoints();
		const int32 NumPoints = ShapePoints.Num();
		const FTransform& LocalToWorld = ShapeComp->GetComponentTransform();

		// Shape control point selection always uses transparent box selection.
		for (int32 Idx = 0; Idx < NumPoints; Idx++)
		{
			const FVector WorldPos = LocalToWorld.TransformPosition(ShapePoints[Idx].Position);
			if (InBox.IsInside(WorldPos))
			{
				ChangeSelectionState(Idx, true);
				bSelectionChanged = true;
			}
		}

		if (bSelectionChanged)
		{
			SelectionState->SetSelectedSegmentIndex(INDEX_NONE);
			SelectionState->SetSelectedControlPoint(INDEX_NONE);
			SelectionState->SetSelectedControlPointType(FZoneShapeControlPointType::None);
		}
	}

	return true;
}

bool FZoneShapeComponentVisualizer::HandleFrustumSelect(const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	const FScopedTransaction Transaction(LOCTEXT("HandleFrustumSelect", "Frustum Select Shape Points"));
	check(SelectionState);
	SelectionState->Modify();

	if (const UZoneShapeComponent* ShapeComp = GetEditedShapeComponent())
	{
		bool bSelectionChanged = false;

		const TConstArrayView<FZoneShapePoint> ShapePoints = ShapeComp->GetPoints();
		const int32 NumPoints = ShapePoints.Num();
		const FTransform& LocalToWorld = ShapeComp->GetComponentTransform();

		// Shape control point selection always uses transparent box selection.
		for (int32 Idx = 0; Idx < NumPoints; Idx++)
		{
			const FVector WorldPos = LocalToWorld.TransformPosition(ShapePoints[Idx].Position);
			if (InFrustum.IntersectPoint(WorldPos))
			{
				ChangeSelectionState(Idx, true);
				bSelectionChanged = true;
			}
		}

		if (bSelectionChanged)
		{
			SelectionState->SetSelectedSegmentIndex(INDEX_NONE);
			SelectionState->SetSelectedControlPoint(INDEX_NONE);
			SelectionState->SetSelectedControlPointType(FZoneShapeControlPointType::None);
		}
	}

	return true;
}

bool FZoneShapeComponentVisualizer::HasFocusOnSelectionBoundingBox(FBox& OutBoundingBox)
{
	OutBoundingBox.Init();

	if (const UZoneShapeComponent* ShapeComp = GetEditedShapeComponent())
	{
		check(SelectionState);
		const TSet<int32>& SelectedPoints = SelectionState->GetSelectedPoints();

		if (SelectedPoints.Num() > 0)
		{
			const TConstArrayView<FZoneShapePoint> ShapePoints = ShapeComp->GetPoints();
			const int32 NumPoints = ShapePoints.Num();
			const FTransform& LocalToWorld = ShapeComp->GetComponentTransform();

			// Shape control point selection always uses transparent box selection.
			for (const int32 Idx : SelectedPoints)
			{
				check(Idx >= 0);
				check(Idx < NumPoints);
				const FVector WorldPos = LocalToWorld.TransformPosition(ShapePoints[Idx].Position);
				OutBoundingBox += WorldPos;
			}

			OutBoundingBox = OutBoundingBox.ExpandBy(50.f);
			return true;
		}
	}

	return false;
}

bool FZoneShapeComponentVisualizer::HandleSnapTo(const bool bInAlign, const bool bInUseLineTrace, const bool bInUseBounds, const bool bInUsePivot, AActor* InDestination)
{
	// Does not handle Snap/Align Pivot, Snap/Align Bottom Control Points or Snap/Align to Actor.
	if (bInUsePivot || bInUseBounds || InDestination)
	{
		return false;
	}

	// Note: value of bInUseLineTrace is ignored as we always line trace from control points.
	if (UZoneShapeComponent* ShapeComp = GetEditedShapeComponent())
	{
		check(SelectionState);
		const TSet<int32>& SelectedPoints = SelectionState->GetSelectedPoints();
		if (SelectedPoints.Num() > 0)
		{
			TArray<FZoneShapePoint>& ShapePoints = ShapeComp->GetMutablePoints();
			const int32 NumPoints = ShapePoints.Num();

			check(SelectionState->GetLastPointIndexSelected() != INDEX_NONE);
			check(SelectionState->GetLastPointIndexSelected() >= 0);
			check(SelectionState->GetLastPointIndexSelected() < NumPoints);
			check(SelectedPoints.Contains(SelectionState->GetLastPointIndexSelected()));

			ShapeComp->Modify();

			bool bMovedKey = false;

			// Shape control point selection always uses transparent box selection.
			for (int32 Idx : SelectedPoints)
			{
				check(Idx >= 0);
				check(Idx < NumPoints);

				FVector Direction = FVector(0.f, 0.f, -1.f);

				FZoneShapePoint& EditedPoint = ShapePoints[Idx];

				FHitResult Hit(1.0f);
				FCollisionQueryParams Params(SCENE_QUERY_STAT(MoveShapePointToTrace), true);

				// Find key position in world space
				const FVector CurrentWorldPos = ShapeComp->GetComponentTransform().TransformPosition(EditedPoint.Position);

				if (ShapeComp->GetWorld()->LineTraceSingleByChannel(Hit, CurrentWorldPos, CurrentWorldPos + Direction * WORLD_MAX, ECC_WorldStatic, Params))
				{
					// Convert back to local space
					EditedPoint.Position = ShapeComp->GetComponentTransform().InverseTransformPosition(Hit.Location);

					if (bInAlign && EditedPoint.Type == FZoneShapePointType::Bezier)
					{
						// Get delta rotation between up vector and hit normal
						FQuat DeltaRotate = FQuat::FindBetweenNormals(FVector::UpVector, Hit.Normal);

						// Rotate tangent according to delta rotation
						const FVector WorldPosition = ShapeComp->GetComponentTransform().TransformPosition(EditedPoint.Position);
						const FVector WorldInControlPoint = ShapeComp->GetComponentTransform().TransformPosition(EditedPoint.GetInControlPoint());
						const FVector WorldTangent = WorldInControlPoint - WorldPosition;
						FVector NewTangent = DeltaRotate.RotateVector(WorldTangent);
						NewTangent = ShapeComp->GetComponentTransform().InverseTransformVector(NewTangent);
						EditedPoint.SetInControlPoint(EditedPoint.Position + NewTangent);
					}

					bMovedKey = true;
				}
			}

			if (bMovedKey)
			{
				ShapeComp->UpdateShape();
				NotifyPropertyModified(ShapeComp, ShapePointsProperty);
				GEditor->RedrawLevelEditingViewports(true);
			}

			return true;
		}
	}

	return false;
}

void FZoneShapeComponentVisualizer::EndEditing()
{
	// Ignore if there is an undo/redo operation in progress
	if (GIsTransacting)
	{
		return;
	}

	// Ignore if this happens during selection.
	if (bIsSelectingComponent)
	{
		return;
	}
	
	check(SelectionState);
	SelectionState->Modify();
	if (GetEditedShapeComponent())
	{
		ChangeSelectionState(INDEX_NONE, false);
		SelectionState->SetSelectedSegmentIndex(INDEX_NONE);
		SelectionState->SetSelectedControlPoint(INDEX_NONE);
		SelectionState->SetSelectedControlPointType(FZoneShapeControlPointType::None);
	}
	SelectionState->SetShapePropertyPath(FComponentPropertyPath());
}

void FZoneShapeComponentVisualizer::OnDuplicatePoint() const
{
	DuplicateSelectedPoints();
}

bool FZoneShapeComponentVisualizer::CanAddPointToSegment() const
{
	if (const UZoneShapeComponent* ShapeComp = GetEditedShapeComponent())
	{
		check(SelectionState);
		const int32 SelectedSegmentIndex = SelectionState->GetSelectedSegmentIndex();
		return (SelectedSegmentIndex != INDEX_NONE && SelectedSegmentIndex >= 0 && SelectedSegmentIndex < ShapeComp->GetNumPoints());
	}
	return false;
}

void FZoneShapeComponentVisualizer::OnAddPointToSegment() const
{
	const FScopedTransaction Transaction(LOCTEXT("AddShapePoint", "Add Shape Point"));
	UZoneShapeComponent* ShapeComp = GetEditedShapeComponent();
	check(ShapeComp != nullptr);
	const int32 SelectedSegmentIndex = SelectionState->GetSelectedSegmentIndex();
	check(SelectionState);
	check(SelectedSegmentIndex != INDEX_NONE);
	check(SelectedSegmentIndex >= 0);
	check(SelectedSegmentIndex < ShapeComp->GetNumSegments());

	SelectionState->Modify();

	SplitSegment(SelectionState->GetSelectedSegmentIndex(), SelectionState->GetSelectedSegmentT());

	SelectionState->SetSelectedSegmentPoint(FVector::ZeroVector);
	SelectionState->SetSelectedSegmentIndex(INDEX_NONE);
}

void FZoneShapeComponentVisualizer::DuplicateSelectedPoints(const FVector& WorldOffset, bool bInsertAfter) const
{
	const FScopedTransaction Transaction(LOCTEXT("DuplicatePoint", "Duplicate Point"));

	UZoneShapeComponent* ShapeComp = GetEditedShapeComponent();
	check(ShapeComp != nullptr);
	check(SelectionState);
	TSet<int32>& SelectedPoints = SelectionState->ModifySelectedPoints();
	const int32 LastPointIndexSelected = SelectionState->GetLastPointIndexSelected();
	check(LastPointIndexSelected != INDEX_NONE);
	check(LastPointIndexSelected >= 0);
	check(LastPointIndexSelected < ShapeComp->GetNumPoints());
	check(SelectedPoints.Num() > 0);
	check(SelectedPoints.Contains(LastPointIndexSelected));

	SelectionState->Modify();

	ShapeComp->Modify();
	if (AActor* Owner = ShapeComp->GetOwner())
	{
		Owner->Modify();
	}

	TArray<int32> SelectedPointsSorted;
	for (int32 SelectedIndex : SelectedPoints)
	{
		SelectedPointsSorted.Add(SelectedIndex);
	}
	SelectedPointsSorted.Sort([](int32 A, int32 B) { return A < B; });

	TArray<FZoneShapePoint>& ShapePoints = ShapeComp->GetMutablePoints();

	// Make copies of the points and adjust them based on the requested offset.
	const FVector LocalOffset = ShapeComp->GetComponentTransform().InverseTransformVector(WorldOffset);
	TArray<FZoneShapePoint> SelectedPointsCopy;
	for (const int32 SelectedIndex : SelectedPointsSorted)
	{
		FZoneShapePoint& Point = SelectedPointsCopy.Add_GetRef(ShapePoints[SelectedIndex]);
		Point.Position += LocalOffset;
	}

	SelectedPoints.Empty();

	// The offset is incremented each time a point to make sure that the following points are inserted at after their copies too.
	int32 Offset = bInsertAfter ? 1 : 0;
	for (int32 i = 0; i < SelectedPointsSorted.Num(); i++)
	{
		// Add new point
		const int32 SelectedIndex = SelectedPointsSorted[i];
		const FZoneShapePoint& Point = SelectedPointsCopy[i];
		const int32 InsertIndex = SelectedIndex + Offset;
		check(InsertIndex <= ShapePoints.Num());
		ShapePoints.Insert(Point, InsertIndex);

		// Adjust selection
		if (LastPointIndexSelected == SelectedIndex)
		{
			SelectionState->SetLastPointIndexSelected(InsertIndex);
		}
		SelectedPoints.Add(InsertIndex);

		Offset++;
	}

	ShapeComp->UpdateShape();

	// Unset tangent handle selection
	SelectionState->SetSelectedControlPoint(INDEX_NONE);
	SelectionState->SetSelectedControlPointType(FZoneShapeControlPointType::None);

	NotifyPropertyModified(ShapeComp, ShapePointsProperty);

	GEditor->RedrawLevelEditingViewports(true);
}

bool FZoneShapeComponentVisualizer::DuplicatePointForAltDrag(const FVector& InDrag) const
{
	UZoneShapeComponent* ShapeComp = GetEditedShapeComponent();
	check(ShapeComp != nullptr);
	check(SelectionState);
	const TSet<int32>& SelectedPoints = SelectionState->GetSelectedPoints();
	const int32 LastPointIndexSelected = SelectionState->GetLastPointIndexSelected();
	const int32 NumPoints = ShapeComp->GetNumPoints();
	check(LastPointIndexSelected != INDEX_NONE);
	check(LastPointIndexSelected >= 0);
	check(LastPointIndexSelected < NumPoints);
	check(SelectedPoints.Contains(LastPointIndexSelected));

	// Calculate approximate tangent around the current point.
	int32 PrevIndex = 0;
	int32 NextIndex = 0;
	if (ShapeComp->IsShapeClosed())
	{
		PrevIndex = (LastPointIndexSelected + NumPoints - 1) % NumPoints;
		NextIndex = (LastPointIndexSelected + 1) % NumPoints;
	}
	else
	{
		PrevIndex = FMath::Max(0, LastPointIndexSelected - 1);
		NextIndex = FMath::Min(LastPointIndexSelected + 1, NumPoints - 1);
	}

	const TConstArrayView<FZoneShapePoint> ShapePoints = ShapeComp->GetPoints();
	const FVector PrevPoint = ShapePoints[PrevIndex].Position;
	const FVector NextPoint = ShapePoints[NextIndex].Position;
	const FVector TangentDir = (NextPoint - PrevPoint).GetSafeNormal();

	// Detect where to insert the point based on if we're dragging towards the next point or previous point.
	const bool bInsertAfter = FVector::DotProduct(TangentDir, InDrag) > 0.0f;

	DuplicateSelectedPoints(InDrag, bInsertAfter);

	return true;
}

void FZoneShapeComponentVisualizer::SplitSegment(const int32 InSegmentIndex, const float SegmentSplitT, UZoneShapeComponent* ShapeComp) const
{
	if (!ShapeComp)
	{
		ShapeComp = GetEditedShapeComponent();
	}

	check(ShapeComp != nullptr);
	check(InSegmentIndex != INDEX_NONE);
	check(InSegmentIndex >= 0);
	check(InSegmentIndex < ShapeComp->GetNumSegments());

	ShapeComp->Modify();
	if (AActor* Owner = ShapeComp->GetOwner())
	{
		Owner->Modify();
	}

	TArray<FZoneShapePoint>& ShapePoints = ShapeComp->GetMutablePoints();
	const int32 NumPoints = ShapePoints.Num();
	const int32 StartPointIdx = InSegmentIndex;
	const int32 EndPointIdx = (InSegmentIndex + 1) % NumPoints;
	const FZoneShapePoint& StartPoint = ShapePoints[StartPointIdx];
	const FZoneShapePoint& EndPoint = ShapePoints[EndPointIdx];

	FVector StartPosition(ForceInitToZero), StartControlPoint(ForceInitToZero), EndControlPoint(ForceInitToZero), EndPosition(ForceInitToZero);
	UE::ZoneShape::Utilities::GetCubicBezierPointsFromShapeSegment(StartPoint, EndPoint, FMatrix::Identity, StartPosition, StartControlPoint, EndControlPoint, EndPosition);

	FZoneShapePoint NewPoint;
	NewPoint.Position = UE::CubicBezier::Eval(StartPosition, StartControlPoint, EndControlPoint, EndPosition, SegmentSplitT);


	// Set new point type based on neighbors
	if (StartPoint.Type == FZoneShapePointType::AutoBezier || EndPoint.Type == FZoneShapePointType::AutoBezier)
	{
		// Auto bezier handles will be updated in UpdateShape()
		NewPoint.Type = FZoneShapePointType::AutoBezier;
	}
	else if (StartPoint.Type == FZoneShapePointType::Bezier || EndPoint.Type == FZoneShapePointType::Bezier)
	{
		// Initial Bezier handles are created below, after insert.
		NewPoint.Type = FZoneShapePointType::Bezier;
	}
	else
	{
		NewPoint.Type = FZoneShapePointType::Sharp;
		NewPoint.TangentLength = 0.0f;
	}

	const int32 NewPointIndex = InSegmentIndex + 1;

	ShapePoints.Insert(NewPoint, NewPointIndex);

	// Create sane default tangent for Bezier points.
	if (NewPoint.Type == FZoneShapePointType::Bezier)
	{
		ShapeComp->UpdatePointRotationAndTangent(NewPointIndex);
	}

	// Set selection to new point
	ChangeSelectionState(NewPointIndex, false);

	ShapeComp->UpdateShape();
	NotifyPropertyModified(ShapeComp, ShapePointsProperty);

	GEditor->RedrawLevelEditingViewports(true);
}

void FZoneShapeComponentVisualizer::AddSegment(const FVector& InWorldPos, const int32 InSelectedIndex, UZoneShapeComponent* InShapeComp) const
{
	if (!InShapeComp)
	{
		InShapeComp = GetEditedShapeComponent();
	}

	check(InShapeComp != nullptr);
	check(InSelectedIndex != INDEX_NONE);
	check(InSelectedIndex >= 0);

	InShapeComp->Modify();
	if (AActor* Owner = InShapeComp->GetOwner())
	{
		Owner->Modify();
	}

	TArray<FZoneShapePoint>& ShapePoints = InShapeComp->GetMutablePoints();
	const int32 NumPoints = ShapePoints.Num();

	const int32 PrevPointIdx = InShapeComp->IsShapeClosed() ? (InSelectedIndex + NumPoints - 1) % NumPoints : InSelectedIndex - 1;
	const int32 NextPointIdx = InShapeComp->IsShapeClosed() ? (InSelectedIndex + 1) % NumPoints : InSelectedIndex + 1;

	FZoneShapePoint NewPoint;
	NewPoint.Position = InShapeComp->GetComponentTransform().InverseTransformPosition(InWorldPos);

	const FZoneShapePoint& SelectedPoint = ShapePoints[InSelectedIndex];
	int32 NewPointIndex = InSelectedIndex + 1; // By default, insert new point after the selected
	
	if (PrevPointIdx >= 0 && NextPointIdx < NumPoints)
	{
		// Both previous and next point are available (selected is neither first nor last)
		// Calculate which segment is closer
		const FZoneShapePoint& PrevPoint = ShapePoints[PrevPointIdx];
		FVector ClosestPointToPrevSegment;
		float PrevSegmentT = 0.0f;
		UE::CubicBezier::ClosestPointApproximate(
			NewPoint.Position,
			SelectedPoint.Position,
			SelectedPoint.GetOutControlPoint(),
			PrevPoint.Position,
			PrevPoint.GetInControlPoint(),
			ClosestPointToPrevSegment,
			PrevSegmentT);

		const FZoneShapePoint& NextPoint = ShapePoints[NextPointIdx];
		FVector ClosestPointToNextSegment;
		float NextSegmentT = 0.0f;
		UE::CubicBezier::ClosestPointApproximate(
			NewPoint.Position,
			SelectedPoint.Position,
			SelectedPoint.GetOutControlPoint(),
			NextPoint.Position,
			NextPoint.GetInControlPoint(),
			ClosestPointToNextSegment,
			NextSegmentT);

		// Insert new point before the selected if the previous segment is closer
		if (FVector::Dist(ClosestPointToPrevSegment, NewPoint.Position) < FVector::Dist(ClosestPointToNextSegment, NewPoint.Position))
		{
			NewPointIndex = InSelectedIndex;
		}
	}
	else if (PrevPointIdx < 0)
	{
		// No previous point (selected is the first) - insert point before selected
		NewPointIndex = InSelectedIndex;
	}

	// Copy the type from a selected point if it's a bezier point
	if (SelectedPoint.Type == FZoneShapePointType::AutoBezier || SelectedPoint.Type == FZoneShapePointType::Bezier)
	{
		NewPoint.Type = SelectedPoint.Type;
	}

	ShapePoints.Insert(NewPoint, NewPointIndex);

	// Create sane default tangent for Bezier points.
	if (NewPoint.Type == FZoneShapePointType::Bezier)
	{
		InShapeComp->UpdatePointRotationAndTangent(NewPointIndex);
	}

	// Set selection to new point
	ChangeSelectionState(NewPointIndex, /*bIsCtrlHeld=*/ false);

	InShapeComp->UpdateShape();
	NotifyPropertyModified(InShapeComp, ShapePointsProperty);

	GEditor->RedrawLevelEditingViewports(/*bInvalidateHitProxies=*/ true);
}

void FZoneShapeComponentVisualizer::OnDeletePoint() const
{
	const FScopedTransaction Transaction(LOCTEXT("DeletePoint", "Delete Points"));
	UZoneShapeComponent* ShapeComp = GetEditedShapeComponent();
	check(ShapeComp != nullptr);
	check(SelectionState);
	const TSet<int32>& SelectedPoints = SelectionState->GetSelectedPoints();
	const int32 LastPointIndexSelected = SelectionState->GetLastPointIndexSelected();
	check(LastPointIndexSelected != INDEX_NONE);
	check(LastPointIndexSelected >= 0);
	check(LastPointIndexSelected < ShapeComp->GetNumPoints());
	check(SelectedPoints.Num() > 0);
	check(SelectedPoints.Contains(LastPointIndexSelected));

	ShapeComp->Modify();
	if (AActor* Owner = ShapeComp->GetOwner())
	{
		Owner->Modify();
	}

	// Get a sorted list of all the selected indices, highest to lowest
	TArray<int32> SelectedPointsSorted;
	for (int32 SelectedIndex : SelectedPoints)
	{
		SelectedPointsSorted.Add(SelectedIndex);
	}
	SelectedPointsSorted.Sort([](int32 A, int32 B) { return A > B; });

	// Delete selected keys from list, highest index first
	TArray<FZoneShapePoint>& ShapePoints = ShapeComp->GetMutablePoints();
	for (const int32 SelectedIndex : SelectedPointsSorted)
	{
		if (ShapePoints.Num() <= 2)
		{
			// Keep at least 2 points
			break;
		}

		ShapePoints.RemoveAt(SelectedIndex);
	}

	// Clear selection
	ChangeSelectionState(INDEX_NONE, false);
	SelectionState->SetSelectedSegmentIndex(INDEX_NONE);
	SelectionState->SetSelectedControlPoint(INDEX_NONE);
	SelectionState->SetSelectedControlPointType(FZoneShapeControlPointType::None);

	ShapeComp->UpdateShape();
	NotifyPropertyModified(ShapeComp, ShapePointsProperty);

	GEditor->RedrawLevelEditingViewports(true);
}


bool FZoneShapeComponentVisualizer::CanDeletePoint() const
{
	check(SelectionState);
	const TSet<int32>& SelectedPoints = SelectionState->GetSelectedPoints();
	const int32 LastPointIndexSelected = SelectionState->GetLastPointIndexSelected();
	UZoneShapeComponent* ShapeComp = GetEditedShapeComponent();
	return (ShapeComp != nullptr &&
		SelectedPoints.Num() > 0 &&
		SelectedPoints.Num() != ShapeComp->GetNumPoints() &&
		LastPointIndexSelected != INDEX_NONE);
}


bool FZoneShapeComponentVisualizer::IsPointSelectionValid() const
{
	check(SelectionState);
	const TSet<int32>& SelectedPoints = SelectionState->GetSelectedPoints();
	const int32 LastPointIndexSelected = SelectionState->GetLastPointIndexSelected();
	UZoneShapeComponent* ShapeComp = GetEditedShapeComponent();
	return (ShapeComp != nullptr &&
		SelectedPoints.Num() > 0 &&
		LastPointIndexSelected != INDEX_NONE);
}

void FZoneShapeComponentVisualizer::OnSetPointType(FZoneShapePointType NewType) const
{
	UZoneShapeComponent* ShapeComp = GetEditedShapeComponent();
	check(ShapeComp != nullptr);
	check(SelectionState);
	const TSet<int32>& SelectedPoints = SelectionState->GetSelectedPoints();

	const FScopedTransaction Transaction(LOCTEXT("SetPointType", "Set Point Type"));

	ShapeComp->Modify();
	if (AActor* Owner = ShapeComp->GetOwner())
	{
		Owner->Modify();
	}

	TArray<FZoneShapePoint>& ShapePoints = ShapeComp->GetMutablePoints();

	for (const int32 SelectedIndex : SelectedPoints)
	{
		check(SelectedIndex >= 0);
		check(SelectedIndex < ShapePoints.Num());

		FZoneShapePoint& Point = ShapePoints[SelectedIndex];
		if (Point.Type != NewType)
		{
			const FZoneShapePointType OldType = Point.Type;
			Point.Type = NewType;
			if (Point.Type == FZoneShapePointType::Sharp)
			{
				Point.TangentLength = 0.0f;
			}
			else if (OldType == FZoneShapePointType::Sharp)
			{
				if (Point.Type == FZoneShapePointType::Bezier || Point.Type == FZoneShapePointType::LaneProfile)
				{
					// Initialize bezier points with auto tangents.
					ShapeComp->UpdatePointRotationAndTangent(SelectedIndex);
				}
			}
			else if (OldType == FZoneShapePointType::LaneProfile && Point.Type != FZoneShapePointType::LaneProfile)
			{
				// Change forward to point along tangent.
				Point.Rotation.Yaw -= 90.0f;
			}
			else if (OldType != FZoneShapePointType::LaneProfile && Point.Type == FZoneShapePointType::LaneProfile)
			{
				// Change forward to point inside the shape.
				Point.Rotation.Yaw += 90.0f;
			}
		}
	}

	ShapeComp->UpdateShape();
	NotifyPropertyModified(ShapeComp, ShapePointsProperty);

	GEditor->RedrawLevelEditingViewports(true);
}


bool FZoneShapeComponentVisualizer::IsPointTypeSet(FZoneShapePointType Type) const
{
	if (IsPointSelectionValid())
	{
		UZoneShapeComponent* ShapeComp = GetEditedShapeComponent();
		check(ShapeComp != nullptr);
		check(SelectionState);
		const TSet<int32>& SelectedPoints = SelectionState->GetSelectedPoints();

		const TConstArrayView<FZoneShapePoint> ShapePoints = ShapeComp->GetPoints();

		for (const int32 SelectedIndex : SelectedPoints)
		{
			check(SelectedIndex >= 0);
			check(SelectedIndex < ShapePoints.Num());
			if (ShapePoints[SelectedIndex].Type == Type)
			{
				return true;
			}
		}
	}

	return false;
}

void FZoneShapeComponentVisualizer::OnSelectAllPoints() const
{
	if (const UZoneShapeComponent* ShapeComp = GetEditedShapeComponent())
	{
		check(SelectionState);
		TSet<int32>& SelectedPoints = SelectionState->ModifySelectedPoints();

		const FScopedTransaction Transaction(LOCTEXT("SelectAllPoints", "Select All Points"));

		SelectionState->Modify();
		SelectedPoints.Empty();

		// Shape control point selection always uses transparent box selection.
		const int32 NumPoints = ShapeComp->GetNumPoints();
		for (int32 Idx = 0; Idx < NumPoints; Idx++)
		{
			SelectedPoints.Add(Idx);
		}

		SelectionState->SetLastPointIndexSelected(NumPoints - 1);
		SelectionState->SetSelectedSegmentIndex(INDEX_NONE);
		SelectionState->SetSelectedControlPoint(INDEX_NONE);
		SelectionState->SetSelectedControlPointType(FZoneShapeControlPointType::None);
	}
}

bool FZoneShapeComponentVisualizer::CanSelectAllPoints() const
{
	UZoneShapeComponent* ShapeComp = GetEditedShapeComponent();
	return (ShapeComp != nullptr);
}

void FZoneShapeComponentVisualizer::OnBreakAtPointNewActors() const
{
	const FScopedTransaction Transaction(LOCTEXT("BreakAtPointNewActors", "Break Shape Into New Actors At Points"));
	BreakAtPoint(true);
}

void FZoneShapeComponentVisualizer::OnBreakAtPointNewComponents() const
{
	const FScopedTransaction Transaction(LOCTEXT("BreakAtPointNewComponents", "Break Shape Into New Components At Points"));
	BreakAtPoint(false);
}

TArray<UZoneShapeComponent*>  FZoneShapeComponentVisualizer::BreakAtPoint(bool bCreateNewActor, UZoneShapeComponent* ShapeComp) const
{
	if (!ShapeComp)
	{
		ShapeComp = GetEditedShapeComponent();
	}
	check(ShapeComp != nullptr);
	check(SelectionState);
	const TSet<int32>& SelectedPoints = SelectionState->GetSelectedPoints();
	const int32 LastPointIndexSelected = SelectionState->GetLastPointIndexSelected();
	check(LastPointIndexSelected != INDEX_NONE);
	check(LastPointIndexSelected >= 0);
	check(LastPointIndexSelected < ShapeComp->GetNumPoints());
	check(SelectedPoints.Num() > 0);
	check(SelectedPoints.Contains(LastPointIndexSelected));

	TArray<UZoneShapeComponent*> ShapeComponents;
	ShapeComponents.Add(ShapeComp);

	ShapeComp->Modify();
	if (AActor* Owner = ShapeComp->GetOwner())
	{
		Owner->Modify();
	}

	// Get a sorted list of all the selected indices, highest to lowest
	TArray<int32> SelectedPointsSorted;
	for (int32 SelectedIndex : SelectedPoints)
	{
		SelectedPointsSorted.Add(SelectedIndex);
	}
	SelectedPointsSorted.Sort([](int32 A, int32 B)
		{ return A < B; });

	// Create a new shape and then delete selected key from list, highest index first
	FActorSpawnParameters SpawnParams;
	TArray<FZoneShapePoint>& ShapePoints = ShapeComp->GetMutablePoints();
	int32 EndIndex = ShapePoints.Num() - 1;
	for (int32 i = SelectedPointsSorted.Num() - 1; i >= 0; i--)
	{
		if (ShapePoints.Num() <= 2)
		{
			// Keep at least 2 points
			break;
		}

		const int32 SelectedIndex = SelectedPointsSorted[i];
		if (SelectedIndex == (ShapePoints.Num() - 1) || SelectedIndex == 0)
		{
			continue;
		}

		// Create a new shape
		UZoneShapeComponent* NewShapeComponent = nullptr;
		AActor* ShapeOwner = ShapeComp->GetOwner();
		if (bCreateNewActor)
		{
			AZoneShape* NewShapeActor = ShapeComp->GetWorld()->SpawnActor<AZoneShape>(AZoneShape::StaticClass(), ShapeComp->GetComponentTransform(), SpawnParams);
			if (!NewShapeActor)
			{
				continue;
			}
			NewShapeComponent = NewShapeActor->GetComponentByClass<UZoneShapeComponent>();
			NewShapeActor->Modify();
		}
		else
		{
			NewShapeComponent = NewObject<UZoneShapeComponent>(ShapeComp->GetOuter(), NAME_None, RF_Transactional);
			if (!NewShapeComponent)
			{
				continue;
			}
			NewShapeComponent->SetWorldTransform(ShapeComp->GetComponentTransform());
			ShapeOwner->AddInstanceComponent(NewShapeComponent);
			NewShapeComponent->RegisterComponent();
			NewShapeComponent->AttachToComponent(ShapeComp, FAttachmentTransformRules::KeepWorldTransform);
			NewShapeComponent->Modify();
		}
		NewShapeComponent->SetCommonLaneProfile(ShapeComp->GetCommonLaneProfile());
		ShapeComponents.Add(NewShapeComponent);

		// Copy points
		TArray<FZoneShapePoint>& NewShapePoints = NewShapeComponent->GetMutablePoints();
		NewShapePoints.SetNum(EndIndex - SelectedIndex + 1);
		int32 SrcIndex = SelectedIndex;
		for (int32 Index = 0; Index < NewShapePoints.Num(); Index++, SrcIndex++)
		{
			NewShapePoints[Index] = ShapePoints[SrcIndex];
		}
		NewShapeComponent->UpdateShape();

		// Delete all points after the selected one
		for (int32 Index = EndIndex; Index > SelectedIndex; Index--)
		{
			if (Index <= 1)
			{
				// The zone shape needs at least two points
				break;
			}
			ShapePoints.RemoveAt(Index);
		}
		EndIndex = SelectedIndex;
	}

	// Clear selection
	ChangeSelectionState(INDEX_NONE, false);
	SelectionState->SetSelectedSegmentIndex(INDEX_NONE);
	SelectionState->SetSelectedControlPoint(INDEX_NONE);
	SelectionState->SetSelectedControlPointType(FZoneShapeControlPointType::None);

	ShapeComp->UpdateShape();
	NotifyPropertyModified(ShapeComp, ShapePointsProperty);

	GEditor->RedrawLevelEditingViewports(true);
	FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditor.BroadcastComponentsEdited();
	LevelEditor.BroadcastRedrawViewports(false);

	return ShapeComponents;
}

bool FZoneShapeComponentVisualizer::CanBreakAtPoint() const
{
	check(SelectionState);
	const TSet<int32>& SelectedPoints = SelectionState->GetSelectedPoints();
	const int32 LastPointIndexSelected = SelectionState->GetLastPointIndexSelected();
	UZoneShapeComponent* ShapeComp = GetEditedShapeComponent();
	return (ShapeComp != nullptr && ShapeComp->GetShapeType() == FZoneShapeType::Spline && SelectedPoints.Num() > 0 && LastPointIndexSelected != INDEX_NONE);
}

void FZoneShapeComponentVisualizer::OnBreakAtSegmentNewActors() const
{
	const FScopedTransaction Transaction(LOCTEXT("BreakAtSegmentNewActors", "Break Shape Into New Actors At The Cursor Location"));
	BreakAtSegment(true);
}

void FZoneShapeComponentVisualizer::OnBreakAtSegmentNewComponents() const
{
	const FScopedTransaction Transaction(LOCTEXT("BreakAtSegmentNewComponents", "Break Shape Into New Components At The Cursor Location"));
	BreakAtSegment(false);
}

void FZoneShapeComponentVisualizer::BreakAtSegment(bool bCreateNewActor) const
{
	UZoneShapeComponent* ShapeComp = GetEditedShapeComponent();
	check(ShapeComp != nullptr);
	const int32 SelectedSegmentIndex = SelectionState->GetSelectedSegmentIndex();
	check(SelectionState);
	check(SelectedSegmentIndex != INDEX_NONE);
	check(SelectedSegmentIndex >= 0);
	check(SelectedSegmentIndex < ShapeComp->GetNumSegments());
	SelectionState->Modify();
	int32 SegmentIndex = SelectionState->GetSelectedSegmentIndex();
	SplitSegment(SegmentIndex, SelectionState->GetSelectedSegmentT());
	const int32 NewPointIndex = SegmentIndex + 1;
	ChangeSelectionState(NewPointIndex, false);
	BreakAtPoint(bCreateNewActor);
	SelectionState->SetSelectedSegmentPoint(FVector::ZeroVector);
	SelectionState->SetSelectedSegmentIndex(INDEX_NONE);
}

bool FZoneShapeComponentVisualizer::CanBreakAtSegment() const
{
	const UZoneShapeComponent* ShapeComp = GetEditedShapeComponent();
	if (ShapeComp != nullptr && ShapeComp->GetShapeType() == FZoneShapeType::Spline)
	{
		check(SelectionState);
		const int32 SelectedSegmentIndex = SelectionState->GetSelectedSegmentIndex();
		return (SelectedSegmentIndex != INDEX_NONE && SelectedSegmentIndex >= 0 && SelectedSegmentIndex < ShapeComp->GetNumPoints());
	}
	return false;
}

TSharedPtr<SWidget> FZoneShapeComponentVisualizer::GenerateContextMenu() const
{
	check(SelectionState);

	FMenuBuilder MenuBuilder(true, ShapeComponentVisualizerActions);

	MenuBuilder.BeginSection("ShapePointEdit", LOCTEXT("ShapePoint", "Shape Point"));
	{
		if (SelectionState->GetSelectedSegmentIndex() != INDEX_NONE)
		{
			MenuBuilder.AddMenuEntry(FZoneShapeComponentVisualizerCommands::Get().AddPoint);

			if (CanBreakAtSegment())
			{
				MenuBuilder.AddSubMenu(
					LOCTEXT("BreakAtPoint", "Break At Point"),
					LOCTEXT("BreakAtPointTooltip", "Break the shape into pieces at the currently selected points."),
					FNewMenuDelegate::CreateSP(this, &FZoneShapeComponentVisualizer::GenerateBreakAtSegmentSubMenu));
			}
		}
		else if (SelectionState->GetLastPointIndexSelected() != INDEX_NONE)
		{
			MenuBuilder.AddMenuEntry(FZoneShapeComponentVisualizerCommands::Get().DeletePoint);
			MenuBuilder.AddMenuEntry(FZoneShapeComponentVisualizerCommands::Get().DuplicatePoint);
			MenuBuilder.AddMenuEntry(FZoneShapeComponentVisualizerCommands::Get().SelectAll);

			MenuBuilder.AddSubMenu(
				LOCTEXT("ShapePointType", "Point Type"),
				LOCTEXT("ShapePointTypeTooltip", "Define the type of the point."),
				FNewMenuDelegate::CreateSP(this, &FZoneShapeComponentVisualizer::GenerateShapePointTypeSubMenu));

			MenuBuilder.AddSubMenu(
				LOCTEXT("SplineSnapAlign", "Snap/Align"),
				LOCTEXT("SplineSnapAlignTooltip", "Snap align options."),
				FNewMenuDelegate::CreateSP(this, &FZoneShapeComponentVisualizer::GenerateSnapAlignSubMenu));

			if (CanBreakAtPoint())
			{
				MenuBuilder.AddSubMenu(
					LOCTEXT("BreakAtPoint", "Break At Point"),
					LOCTEXT("BreakAtPointTooltip", "Break the shape into pieces at the currently selected points."),
					FNewMenuDelegate::CreateSP(this, &FZoneShapeComponentVisualizer::GenerateBreakAtPointSubMenu));
			}
		}
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Transform");
	{
		MenuBuilder.AddMenuEntry(FZoneShapeComponentVisualizerCommands::Get().FocusViewportToSelection);
	}
	MenuBuilder.EndSection();

	TSharedPtr<SWidget> MenuWidget = MenuBuilder.MakeWidget();
	return MenuWidget;
}

void FZoneShapeComponentVisualizer::GenerateShapePointTypeSubMenu(FMenuBuilder& MenuBuilder) const
{
	UZoneShapeComponent* ShapeComp = GetEditedShapeComponent();

	MenuBuilder.AddMenuEntry(FZoneShapeComponentVisualizerCommands::Get().SetPointToSharp);
	MenuBuilder.AddMenuEntry(FZoneShapeComponentVisualizerCommands::Get().SetPointToBezier);
	MenuBuilder.AddMenuEntry(FZoneShapeComponentVisualizerCommands::Get().SetPointToAutoBezier);
	if (ShapeComp && ShapeComp->GetShapeType() == FZoneShapeType::Polygon)
	{
		MenuBuilder.AddMenuEntry(FZoneShapeComponentVisualizerCommands::Get().SetPointToLaneSegment);
	}
}

void FZoneShapeComponentVisualizer::GenerateSnapAlignSubMenu(FMenuBuilder& MenuBuilder) const
{
	MenuBuilder.AddMenuEntry(FLevelEditorCommands::Get().SnapToFloor);
	MenuBuilder.AddMenuEntry(FLevelEditorCommands::Get().AlignToFloor);
}

void FZoneShapeComponentVisualizer::GenerateBreakAtPointSubMenu(FMenuBuilder& MenuBuilder) const
{
	MenuBuilder.AddMenuEntry(FZoneShapeComponentVisualizerCommands::Get().BreakAtPointNewActors);
	MenuBuilder.AddMenuEntry(FZoneShapeComponentVisualizerCommands::Get().BreakAtPointNewComponents);
}

void FZoneShapeComponentVisualizer::GenerateBreakAtSegmentSubMenu(FMenuBuilder& MenuBuilder) const
{
	UZoneShapeComponent* ShapeComp = GetEditedShapeComponent();
	if (ShapeComp && ShapeComp->GetShapeType() == FZoneShapeType::Spline)
	{
		MenuBuilder.AddMenuEntry(FZoneShapeComponentVisualizerCommands::Get().BreakAtSegmentNewActors);
		MenuBuilder.AddMenuEntry(FZoneShapeComponentVisualizerCommands::Get().BreakAtSegmentNewComponents);
	}
}

void FZoneShapeComponentVisualizer::CreateIntersection(UZoneShapeComponent* ShapeComp)
{
	if (UZoneShapeComponent* TargetShapeComponent = CreateIntersectionState.WeakTargetShapeComponent.Get())
	{
		const FScopedTransaction Transaction(LOCTEXT("CreateIntersection", "Create an Intersection With The Dragged Point and Overlapped Shape"));
		TargetShapeComponent->Modify();
		FZoneShapePoint& DraggedPoint = ShapeComp->GetMutablePoints()[SelectedPointForConnecting];
		if (TargetShapeComponent->GetShapeType() == FZoneShapeType::Spline)
		{
			CreateIntersectionForSplineShape(ShapeComp, DraggedPoint);
		}
		else
		{
			CreateIntersectionForPolygonShape(ShapeComp, DraggedPoint);
		}
	}
}

void FZoneShapeComponentVisualizer::CreateIntersectionForSplineShape(UZoneShapeComponent* ShapeComp, FZoneShapePoint& DraggedPoint, bool DestroyCoveredShape)
{
	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
	if (!ZoneGraphSettings)
	{
		return;
	}

	UZoneShapeComponent* TargetShapeComponent = CreateIntersectionState.WeakTargetShapeComponent.Get();
	if (!TargetShapeComponent)
	{
		return;
	}
	
	if (CreateIntersectionState.OverlappingSegmentIndex == INDEX_NONE)
	{
		return;
	}

	const FZoneLaneProfile* LaneProfile = ZoneGraphSettings->GetLaneProfileByRef(ShapeComp->GetCommonLaneProfile());
	double const HalfLanesTotalWidth = LaneProfile ? LaneProfile->GetLanesTotalWidth() * 0.5 : 0.0;

	// Get overlapping position on the target segment
	FVector NewPointPosition = UE::ZoneGraph::Editor::Private::GetPositionOnSegment(TargetShapeComponent->GetMutablePoints(), CreateIntersectionState.OverlappingSegmentIndex, CreateIntersectionState.OverlappingSegmentT);

	bool bCloseToPoint = CreateIntersectionState.ClosePointIndex != INDEX_NONE;
	if (bCloseToPoint)
	{
		// If close to a point, select it as the point to break at.
		ChangeSelectionState(CreateIntersectionState.ClosePointIndex, false);
	}
	else
	{
		// At the overlapping position, add a point to break at.
		SplitSegment(CreateIntersectionState.OverlappingSegmentIndex, CreateIntersectionState.OverlappingSegmentT, TargetShapeComponent);
	}

	// Break the zone shape
	const FTransform& ShapeCompTransform = ShapeComp->GetComponentTransform();
	TArray<UZoneShapeComponent*> ShapeComponents = BreakAtPoint(true, TargetShapeComponent);

	// Create an intersection
	FActorSpawnParameters SpawnParams;
	AZoneShape* IntersectionShapeActor = ShapeComp->GetWorld()->SpawnActor<AZoneShape>(AZoneShape::StaticClass(), ShapeCompTransform, SpawnParams);
	UZoneShapeComponent* IntersectionShapeComponent = IntersectionShapeActor->GetComponentByClass<UZoneShapeComponent>();
	IntersectionShapeComponent->SetShapeType(FZoneShapeType::Polygon);
	const FTransform& IntersectionTransform = IntersectionShapeComponent->GetComponentTransform();

	FVector Normal = ShapeComp->GetShapeConnectorByPointIndex(SelectedPointForConnecting)->Normal;
	if (ShapeComponents.Num() == 1 && bCloseToPoint)
	{
		// The point was dragged to the start or end of a zone shape. Create an intersection that connects these two shapes.

		// Get the target zone shape's connector that is close to the dragged point.
		int32 PointIndex = CreateIntersectionState.OverlappingSegmentT < 0.5f ? CreateIntersectionState.OverlappingSegmentIndex : CreateIntersectionState.OverlappingSegmentIndex + 1;
		const FZoneShapeConnector* TargetConnector = TargetShapeComponent->GetShapeConnectorByPointIndex(PointIndex);

		// Compute the intersection location from the connector position and normal
		FVector TargetNormal = TargetConnector->Normal;
		FVector TargetWorldNormal = TargetShapeComponent->GetComponentTransform().TransformVector(TargetConnector->Normal);
		IntersectionShapeActor->SetActorLocation(NewPointPosition + TargetWorldNormal * HalfLanesTotalWidth);


		// Connect
		TArray<FZoneShapePoint>& Points0 = ShapeComponents[0]->GetMutablePoints();
		int32 Index0 = Points0.Num() - 1;
		FVector Normal0 = ShapeComponents[0]->GetShapeConnectorByPointIndex(Index0)->Normal;
		Points0.Last().Position -= Normal0 * HalfLanesTotalWidth;
		ShapeComponents[0]->UpdateShape();

		TArray<FZoneShapePoint>& Points = IntersectionShapeComponent->GetMutablePoints();
		UE::ZoneGraph::Editor::Private::SetPolygonPointLaneProfileToMatchSpline(Points[0], IntersectionShapeComponent, ShapeComponents[0]);
		TConstArrayView<FZoneShapePoint> TargetPoints = TargetShapeComponent->GetPoints();
		const FTransform Shape0Transform = ShapeComponents[0]->GetComponentTransform();
		const FVector Point0WorldPosition = Shape0Transform.TransformPosition(TargetPoints[PointIndex].Position);
		const FVector Point0WorldNormal = Shape0Transform.TransformVector(TargetNormal);
		UE::ZoneGraph::Editor::Private::SetPointPositionRotation(Points[0], IntersectionTransform, Point0WorldPosition, Point0WorldNormal);

		UE::ZoneGraph::Editor::Private::SetPolygonPointLaneProfileToMatchSpline(Points[1], IntersectionShapeComponent, ShapeComp);
		DraggedPoint.Position -= Normal * HalfLanesTotalWidth;
		const FVector Point1WorldPosition = ShapeCompTransform.TransformPosition(DraggedPoint.Position);
		const FVector Point1WorldNormal = ShapeCompTransform.TransformVector(Normal);
		UE::ZoneGraph::Editor::Private::SetPointPositionRotation(Points[1], IntersectionTransform, Point1WorldPosition, Point1WorldNormal);

		// Update shape
		ShapeComp->UpdateShape();

		// Update point positions
		IntersectionShapeComponent->UpdateShape();
	}
	else if (ShapeComponents.Num() == 2)
	{
		// Cut the intersected shape
		FVector DraggedPointWorldPosition = ShapeCompTransform.TransformPosition(DraggedPoint.Position);
		const FTransform& TargetTransform = TargetShapeComponent->GetComponentTransform();
		FBox Bounds = FBox::BuildAABB(TargetTransform.TransformPosition(NewPointPosition), FVector(HalfLanesTotalWidth));
		// Move points
		const FTransform& ShapeTransform0 = ShapeComponents[0]->GetComponentTransform();
		const FTransform& ShapeTransform1 = ShapeComponents[1]->GetComponentTransform();
		TArray<FZoneShapePoint>& Points0 = ShapeComponents[0]->GetMutablePoints();
		int32 Index0 = Points0.Num() - 1;
		for (int32 i = Index0 - 1; i > 1; i--)
		{
			if (!Bounds.IsInside(ShapeTransform0.TransformPosition(Points0[i].Position)))
			{
				continue;
			}
			Points0.RemoveAt(i);
		}
		Index0 = Points0.Num() - 1;
		if (ShapeComponents[0])
		{
			// Need to update the shape as points could be removed
			// This will rebuild the shape connectors before the GetShapeConnectorByPointIndex call
			ShapeComponents[0]->UpdateShape();
			
			FVector Normal0 = ShapeComponents[0]->GetShapeConnectorByPointIndex(Index0)->Normal;
			FVector Offset = Normal0 * HalfLanesTotalWidth;
			if (Points0.Num() == 2)
			{
				double Length = FVector::Dist(Points0[0].Position, Points0[1].Position);
				if (Length < HalfLanesTotalWidth * 2)
				{
					Offset = Normal0 * Length * 0.5;
				}
			}
			Points0.Last().Position -= Offset;
			ShapeComponents[0]->UpdateShape();
		}

		TArray<FZoneShapePoint>& Points1 = ShapeComponents[1]->GetMutablePoints();
		int32 Index1 = 0;
		for (int32 i = Index1 + 1; i < (Points1.Num() - 2); i++)
		{
			if (!Bounds.IsInside(ShapeTransform1.TransformPosition(Points1[i].Position)))
			{
				continue;
			}
			Points1.RemoveAt(i);
		}
		if (ShapeComponents[1])
		{
			// Need to update the shape as points could be removed
			// This will rebuild the shape connectors before the GetShapeConnectorByPointIndex call
			ShapeComponents[1]->UpdateShape();
			
			FVector Normal1 = ShapeComponents[1]->GetShapeConnectorByPointIndex(Index1)->Normal;
			FVector Offset = Normal1 * HalfLanesTotalWidth;
			if (Points1.Num() == 2)
			{
				double Length = FVector::Dist(Points1[0].Position, Points1[1].Position);
				if (Length < HalfLanesTotalWidth * 2)
				{
					Offset = Normal1 * Length * 0.5;
				}
			}
			Points1[Index1].Position -= Offset;
			ShapeComponents[1]->UpdateShape();
		}

		// Create intersection with the same profile
		IntersectionShapeActor->SetActorLocation(DraggedPointWorldPosition);

		// Get points. Set positions. Set profile.
		TArray<FZoneShapePoint>& Points = IntersectionShapeComponent->GetMutablePoints();
		if (ShapeComponents[0] && ShapeComponents[1])
		{
			Points.Add(FZoneShapePoint(Points[1]));
		}

		// Connect
		int32 IntersectionPointIndex = 0;
		if (ShapeComponents[0])
		{
			UE::ZoneGraph::Editor::Private::SetPolygonPointLaneProfileToMatchSpline(Points[IntersectionPointIndex], IntersectionShapeComponent, ShapeComponents[0]);

			const FVector PointWorldPosition = ShapeTransform0.TransformPosition(Points0.Last().Position);
			const FZoneShapeConnector* Connector0 = ShapeComponents[0]->GetShapeConnectorByPointIndex(Points0.Num() - 1);
			const FVector PointWorldNormal = ShapeTransform0.TransformVector(Connector0->Normal);
			UE::ZoneGraph::Editor::Private::SetPointPositionRotation(Points[IntersectionPointIndex], IntersectionTransform, PointWorldPosition, PointWorldNormal);
			IntersectionPointIndex++;
		}

		if (ShapeComponents[1])
		{
			UE::ZoneGraph::Editor::Private::SetPolygonPointLaneProfileToMatchSpline(Points[IntersectionPointIndex], IntersectionShapeComponent, ShapeComponents[1]);

			const FVector PointWorldPosition = ShapeTransform1.TransformPosition(Points1[0].Position);
			const FZoneShapeConnector* Connector1 = ShapeComponents[1]->GetShapeConnectorByPointIndex(0);
			const FVector PointWorldNormal = ShapeTransform1.TransformVector(Connector1->Normal);
			UE::ZoneGraph::Editor::Private::SetPointPositionRotation(Points[IntersectionPointIndex], IntersectionTransform, PointWorldPosition, PointWorldNormal);
			IntersectionPointIndex++;
		}

		UE::ZoneGraph::Editor::Private::SetPolygonPointLaneProfileToMatchSpline(Points[IntersectionPointIndex], IntersectionShapeComponent, ShapeComp);
		DraggedPoint.Position -= Normal * HalfLanesTotalWidth;
		ShapeComp->UpdateShape(); // Update shape
		DraggedPointWorldPosition = ShapeCompTransform.TransformPosition(DraggedPoint.Position);
		const FVector WorldNormal = ShapeCompTransform.TransformVector(Normal);
		UE::ZoneGraph::Editor::Private::SetPointPositionRotation(Points[IntersectionPointIndex], IntersectionTransform, DraggedPointWorldPosition, WorldNormal);

		UE::ZoneGraph::Editor::Private::SortPolygonPointsCounterclockwise(IntersectionShapeComponent);

		// Update point positions
		IntersectionShapeComponent->UpdateShape();
	}
}

void FZoneShapeComponentVisualizer::CreateIntersectionForPolygonShape(UZoneShapeComponent* ShapeComp, FZoneShapePoint& DraggedPoint)
{
	UZoneShapeComponent* TargetShapeComponent = CreateIntersectionState.WeakTargetShapeComponent.Get();
	if (!TargetShapeComponent)
	{
		return;
	}

	const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
	if (!ZoneGraphSettings)
	{
		return;
	}

	const FTransform& TargetShapeCompTransform = TargetShapeComponent->GetComponentTransform();
	const FZoneGraphBuildSettings& BuildSettings = ZoneGraphSettings->GetBuildSettings();
	TArray<FZoneShapePoint>& TargetShapePoints = TargetShapeComponent->GetMutablePoints();

	const FTransform& SourceTransform = ShapeComp->GetComponentTransform();
	const FZoneShapeConnector* SourceConnector = ShapeComp->GetShapeConnectorByPointIndex(SelectedPointForConnecting);
	if (CreateIntersectionState.ClosePointIndex != INDEX_NONE)
	{
		// If the dragged point is close to a connector, connect.
		const FVector TargetPointWorldPosition = TargetShapeCompTransform.TransformPosition(TargetShapePoints[CreateIntersectionState.ClosePointIndex].Position);
		const FZoneShapeConnector* TargetConnector = TargetShapeComponent->GetShapeConnectorByPointIndex(CreateIntersectionState.ClosePointIndex);
		const FVector TargetPointWorldNormal = TargetShapeComponent->GetComponentTransform().TransformVector(TargetConnector->Normal);

		const double ConnectionSnapAngleCos = FMath::Cos(FMath::DegreesToRadians(BuildSettings.ConnectionSnapAngle));
		const FZoneLaneProfile* LaneProfile = ZoneGraphSettings->GetLaneProfileByRef(ShapeComp->GetCommonLaneProfile());
		const double HalfLanesTotalWidth = LaneProfile ? LaneProfile->GetLanesTotalWidth() * 0.5 : 0.0;
		UE::ZoneGraph::Editor::Private::SnapConnect(
			ShapeComp,
			DraggedPoint,
			SourceTransform,
			SourceTransform.TransformVector(SourceConnector->Normal),
			TargetPointWorldPosition,
			TargetPointWorldNormal,
			ConnectionSnapAngleCos,
			HalfLanesTotalWidth);
	}
	else
	{
		// If the dragged point is not close to any connector, add a point and connect.
		FZoneShapePoint NewPoint = FZoneShapePoint(TargetShapePoints[0]);

		UE::ZoneGraph::Editor::Private::SetPolygonPointLaneProfileToMatchSpline(NewPoint, TargetShapeComponent, ShapeComp);
		TargetShapePoints.Add(NewPoint);
		
		UE::ZoneGraph::Editor::Private::SetPointPositionRotation(
			TargetShapePoints.Last(0),
			TargetShapeCompTransform,
			SourceTransform.TransformPosition(DraggedPoint.Position),
			SourceTransform.TransformVector(SourceConnector->Normal));
		
		UE::ZoneGraph::Editor::Private::SortPolygonPointsCounterclockwise(TargetShapeComponent);
		
		TargetShapeComponent->UpdateShape();
	}
}

void FZoneShapeComponentVisualizer::ClearAutoConnectingStatus()
{
	bIsAutoConnecting = false;
	AutoConnectState = {};
}

void FZoneShapeComponentVisualizer::ClearAutoIntersectionStatus()
{
	bIsCreatingIntersection = false;
	CreateIntersectionState = {};
}

bool FZoneShapeComponentVisualizer::CanAutoConnect(const UZoneShapeComponent* ShapeComp) const
{
	return ShapeComp->GetShapeType() == FZoneShapeType::Spline;
}

bool FZoneShapeComponentVisualizer::CanAutoCreateIntersection(const UZoneShapeComponent* ShapeComp) const
{
	return ShapeComp->GetShapeType() == FZoneShapeType::Spline;
}

#undef LOCTEXT_NAMESPACE
