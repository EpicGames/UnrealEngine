// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentVisualizer.h"
#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "HitProxies.h"
#include "ComponentVisualizer.h"
#include "UObject/GCObject.h"
#include "ZoneShapeComponent.h"
#include "ZoneShapeComponentVisualizer.generated.h"

#define UE_API ZONEGRAPHEDITOR_API

class AActor;
class FEditorViewportClient;
class FMenuBuilder;
class FPrimitiveDrawInterface;
class FSceneView;
class FUICommandList;
class FViewport;
class SWidget;
struct FViewportClick;
struct FConvexVolume;

UENUM()
enum class FZoneShapeControlPointType : uint8
{
	None,
	In,
	Out,
};

/** Selection state data that will be captured by scoped transactions.*/
UCLASS(MinimalAPI, Transient)
class UZoneShapeComponentVisualizerSelectionState : public UObject
{
	GENERATED_BODY()
public:
	FComponentPropertyPath GetShapePropertyPath() const { return ShapePropertyPath; }
	void SetShapePropertyPath(const FComponentPropertyPath& InShapePropertyPath) { ShapePropertyPath = InShapePropertyPath; }
	const TSet<int32>& GetSelectedPoints() const { return SelectedPoints; }
	TSet<int32>& ModifySelectedPoints() { return SelectedPoints; }
	int32 GetLastPointIndexSelected() const { return LastPointIndexSelected; }
	void SetLastPointIndexSelected(const int32 InLastPointIndexSelected) { LastPointIndexSelected = InLastPointIndexSelected; }
	int32 GetSelectedSegmentIndex() const { return SelectedSegmentIndex; }
	void SetSelectedSegmentIndex(const int32 InSelectedSegmentIndex) { SelectedSegmentIndex = InSelectedSegmentIndex; }
	FVector GetSelectedSegmentPoint() const { return SelectedSegmentPoint; }
	void SetSelectedSegmentPoint(const FVector& InSelectedSegmentPoint) { SelectedSegmentPoint = InSelectedSegmentPoint; }
	float GetSelectedSegmentT() const { return SelectedSegmentT; }
	void SetSelectedSegmentT(const float InSelectedSegmentT) { SelectedSegmentT = InSelectedSegmentT; }
	int32 GetSelectedControlPoint() const { return SelectedControlPoint; }
	void SetSelectedControlPoint(const int32 InSelectedControlPoint) { SelectedControlPoint = InSelectedControlPoint; }
	FZoneShapeControlPointType GetSelectedControlPointType() const { return SelectedControlPointType; }
	void SetSelectedControlPointType(const FZoneShapeControlPointType InSelectedControlPointType) { SelectedControlPointType = InSelectedControlPointType; }

protected:
	/** Property path from the parent actor to the component */
	UPROPERTY()
	FComponentPropertyPath ShapePropertyPath;
	/** Index of keys we have selected */
	UPROPERTY()
	TSet<int32> SelectedPoints;
	/** Index of the last key we selected */
	UPROPERTY()
	int32 LastPointIndexSelected = 0;
	/** Index of segment we have selected */
	UPROPERTY()
	int32 SelectedSegmentIndex = 0;
	/** Position on selected segment */
	UPROPERTY()
	FVector SelectedSegmentPoint = FVector::ZeroVector;
	/** Interpolation value along the selected segment */
	UPROPERTY()
	float SelectedSegmentT = 0.0f;
	/** Index of tangent handle we have selected */
	UPROPERTY()
	int32 SelectedControlPoint = 0;
	/** The type of the selected tangent handle */
	UPROPERTY()
	FZoneShapeControlPointType SelectedControlPointType = FZoneShapeControlPointType::None;
};

/** Base class for clickable shape editing proxies */
struct HZoneShapeVisProxy : public HComponentVisProxy
{
	DECLARE_HIT_PROXY( UE_API );

	HZoneShapeVisProxy(const UActorComponent* InComponent, EHitProxyPriority InPriority = HPP_Wireframe)
		: HComponentVisProxy(InComponent, InPriority)
	{}

	virtual EMouseCursor::Type GetMouseCursor() override
	{
		return EMouseCursor::CardinalCross;
	}
};

/** Proxy for a shape point */
struct HZoneShapePointProxy : public HZoneShapeVisProxy
{
	DECLARE_HIT_PROXY( UE_API );

	HZoneShapePointProxy(const UActorComponent* InComponent, int32 InPointIndex)
		: HZoneShapeVisProxy(InComponent, HPP_Foreground)
		, PointIndex(InPointIndex)
	{}

	virtual EMouseCursor::Type GetMouseCursor() override
	{
		return EMouseCursor::CardinalCross;
	}

	int32 PointIndex;
};

/** Proxy for a shape segment */
struct HZoneShapeSegmentProxy : public HZoneShapeVisProxy
{
	DECLARE_HIT_PROXY( UE_API );

	HZoneShapeSegmentProxy(const UActorComponent* InComponent, int32 InSegmentIndex)
		: HZoneShapeVisProxy(InComponent)
		, SegmentIndex(InSegmentIndex)
	{}

	virtual EMouseCursor::Type GetMouseCursor() override
	{
		return EMouseCursor::CardinalCross;
	}

	int32 SegmentIndex;
};

/** Proxy for a control point handle */
struct HZoneShapeControlPointProxy : public HZoneShapeVisProxy
{
	DECLARE_HIT_PROXY( UE_API );

	HZoneShapeControlPointProxy(const UActorComponent* InComponent, int32 InPointIndex, bool bInInControlPoint)
		: HZoneShapeVisProxy(InComponent, HPP_Foreground)
		, PointIndex(InPointIndex)
		, bInControlPoint(bInInControlPoint)
	{}

	virtual EMouseCursor::Type GetMouseCursor() override
	{
		return EMouseCursor::CardinalCross;
	}

	int32 PointIndex;
	bool bInControlPoint;
};

/** ZoneShapeComponent visualizer/edit functionality */
class FZoneShapeComponentVisualizer : public FComponentVisualizer, public FGCObject
{
public:
	UE_API FZoneShapeComponentVisualizer();
	UE_API virtual ~FZoneShapeComponentVisualizer();

	//~ Begin FComponentVisualizer Interface
	UE_API virtual void OnRegister() override;
	UE_API virtual void DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	UE_API virtual bool VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* VisProxy, const FViewportClick& Click) override;
	UE_API virtual void DrawVisualizationHUD(const UActorComponent* Component, const FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) override;
	UE_API virtual void EndEditing() override;
	UE_API virtual bool GetWidgetLocation(const FEditorViewportClient* ViewportClient, FVector& OutLocation) const override;
	UE_API virtual bool GetCustomInputCoordinateSystem(const FEditorViewportClient* ViewportClient, FMatrix& OutMatrix) const override;
	UE_API virtual bool HandleInputDelta(FEditorViewportClient* ViewportClient, FViewport* Viewport, FVector& DeltaTranslate, FRotator& DeltaRotate, FVector& DeltaScale) override;
	UE_API virtual bool HandleInputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event) override;
	/** Handle box select input */
	UE_API virtual bool HandleBoxSelect(const FBox& InBox, FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	/** Handle frustum select input */
	UE_API virtual bool HandleFrustumSelect(const FConvexVolume& InFrustum, FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	/** Return whether focus on selection should focus on bounding box defined by active visualizer */
	UE_API virtual bool HasFocusOnSelectionBoundingBox(FBox& OutBoundingBox) override;
	/** Pass snap input to active visualizer */
	UE_API virtual bool HandleSnapTo(const bool bInAlign, const bool bInUseLineTrace, const bool bInUseBounds, const bool bInUsePivot, AActor* InDestination) override;
	/** Get currently edited component, this is needed to reset the active visualizer after undo/redo */
	UE_API virtual UActorComponent* GetEditedComponent() const override;
	UE_API virtual TSharedPtr<SWidget> GenerateContextMenu() const override;
	UE_API virtual bool IsVisualizingArchetype() const override;
	//~ End FComponentVisualizer Interface

	/** Get the shape component we are currently editing */
	UE_API UZoneShapeComponent* GetEditedShapeComponent() const;

	const TSet<int32>& GetSelectedPoints() const
	{
		check(SelectionState);
		return SelectionState->GetSelectedPoints();
	}

protected:

	struct ZoneShapeConnectorRenderInfo
	{
		ZoneShapeConnectorRenderInfo(FVector InPosition, FVector InFoward, FVector InUp)
			: Position(InPosition)
			, Foward(InFoward)
			, Up(InUp)
		{

		}

		// Position of the connector.
		FVector Position = FVector::ZeroVector;

		// Foward direction of the connector.
		FVector Foward = FVector::ForwardVector;

		// Up direction of the connector.
		FVector Up = FVector::UpVector;
	};

	/** Determine if any selected key index is out of range (perhaps because something external has modified the shape) */
	UE_API bool IsAnySelectedPointIndexOutOfRange(const UZoneShapeComponent& Comp) const;

	/** Whether a single point is currently selected */
	UE_API bool IsSinglePointSelected() const;

	/** Transforms selected control point by given translation */
	UE_API bool TransformSelectedControlPoint(const FVector& DeltaTranslate);

	/** Transforms selected points by given delta translation, rotation and scale. */
	UE_API bool TransformSelectedPoints(const FEditorViewportClient* ViewportClient, const FVector& DeltaTranslate, const FRotator& DeltaRotate, const FVector& DeltaScale) const;

	/** Update the key selection state of the visualizer */
	UE_API void ChangeSelectionState(int32 Index, bool bIsCtrlHeld) const;

	/** Alt-drag: duplicates the selected point */
	UE_API bool DuplicatePointForAltDrag(const FVector& InDrag) const;

	/** Split segment using given interpolation value, selects the new point */
	UE_API void SplitSegment(const int32 InSegmentIndex, const float SegmentSplitT, UZoneShapeComponent* ShapeComp = nullptr) const;

	/** Add segment using given position and index, selects the new point */
	UE_API void AddSegment(const FVector& InWorldPos, const int32 InSelectedIndex, UZoneShapeComponent* InShapeComp = nullptr) const;

	/** Duplicates selected points and selects them. */
	UE_API void DuplicateSelectedPoints(const FVector& WorldOffset = FVector::ZeroVector, bool bInsertAfter = true) const;

	/** Updates the component and selected properties if the component has changed */
	UE_API const UZoneShapeComponent* UpdateSelectedShapeComponent(const HComponentVisProxy* VisProxy);

	/** Returns the rotation of last selected point, or false if last selection is not valid. */
	UE_API bool GetLastSelectedPointRotation(FQuat& OutRotation) const;

	UE_API void OnDeletePoint() const;
	UE_API bool CanDeletePoint() const;

	/** Duplicates selected points in place */
	UE_API void OnDuplicatePoint() const;
	UE_API bool IsPointSelectionValid() const;

	UE_API void OnAddPointToSegment() const;
	UE_API bool CanAddPointToSegment() const;

	UE_API void OnSetPointType(FZoneShapePointType Type) const;
	UE_API bool IsPointTypeSet(FZoneShapePointType Type) const;

	UE_API void OnSelectAllPoints() const;
	UE_API bool CanSelectAllPoints() const;

	UE_API void OnBreakAtPointNewActors() const;
	UE_API void OnBreakAtPointNewComponents() const;
	UE_API bool CanBreakAtPoint() const;

	UE_API void OnBreakAtSegmentNewActors() const;
	UE_API void OnBreakAtSegmentNewComponents() const;
	UE_API bool CanBreakAtSegment() const;

	UE_API void GenerateShapePointTypeSubMenu(FMenuBuilder& MenuBuilder) const;

	UE_API void GenerateSnapAlignSubMenu(FMenuBuilder& MenuBuilder) const;
	UE_API void GenerateBreakAtPointSubMenu(FMenuBuilder& MenuBuilder) const;
	UE_API void GenerateBreakAtSegmentSubMenu(FMenuBuilder& MenuBuilder) const;

	// FGCObject interface
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FZoneShapeComponentVisualizer");
	}
	// End of FGCObject interface

	/** Output log commands */
	TSharedPtr<FUICommandList> ShapeComponentVisualizerActions;

	/** Property to be notified when points change */
	FProperty* ShapePointsProperty;

	/** Current selection state */
	TObjectPtr<UZoneShapeComponentVisualizerSelectionState> SelectionState;

	/** Whether we currently allow duplication when dragging */
	bool bAllowDuplication;

	/** Alt-drag: Accumulates delayed drag offset. */
	FVector DuplicateAccumulatedDrag;

	/** True if the ControlPointPosition has been initialized */
	bool bControlPointPositionCaptured;

	/** Selected control pints adjusted position, allows the gizmo to move freely, while we constrain the control point. */
	FVector ControlPointPosition;

	/** True if cached rotation is set. */
	bool bHasCachedRotation = false;

	/** Rotation cached when mouse button is pressed. */
	FQuat CachedRotation = FQuat::Identity;

	bool bIsSelectingComponent = false;

	/** The point used to calculate the auto connect and intersection states. */
	int32 SelectedPointForConnecting = -1;

	bool bIsAutoConnecting = false;
	struct FAutoConnectState
	{
		TArray<ZoneShapeConnectorRenderInfo> DestShapeConnectorInfos;
		int32 ClosestShapeConnectorInfoIndex = INDEX_NONE;
		FVector NearestPointWorldPosition = FVector::ZeroVector;
		FVector NearestPointWorldNormal = FVector::ZeroVector;
	};
	FAutoConnectState AutoConnectState;

	bool bIsCreatingIntersection = false;
	struct FCreateIntersectionState
	{
		TWeakObjectPtr<UZoneShapeComponent> WeakTargetShapeComponent;
		int32 OverlappingSegmentIndex = INDEX_NONE;
		float OverlappingSegmentT = -1.0f;
		int32 ClosePointIndex = INDEX_NONE;
		FVector PreviewLocation = FVector::ZeroVector;
	};
	FCreateIntersectionState CreateIntersectionState;

private:

	UE_API TArray<UZoneShapeComponent*>  BreakAtPoint(bool bCreateNewActor, UZoneShapeComponent* ShapeComp = nullptr) const;
	UE_API void BreakAtSegment(bool bCreateNewActor) const;

	UE_API void DetectCloseByShapeForAutoConnection(const UZoneShapeComponent* ShapeComp, const FZoneShapePoint& DraggedPoint);
	UE_API void DetectCloseByShapeForAutoIntersectionCreation(const UZoneShapeComponent* ShapeComp, const FZoneShapePoint& DraggedPoint);

	UE_API void CreateIntersection(UZoneShapeComponent* ShapeComp);
	UE_API void CreateIntersectionForSplineShape(UZoneShapeComponent* ShapeComp, FZoneShapePoint& DraggedPoint, bool DestroyCoveredShape = true);
	UE_API void CreateIntersectionForPolygonShape(UZoneShapeComponent* ShapeComp, FZoneShapePoint& DraggedPoint);

	UE_API void ClearAutoConnectingStatus();
	UE_API void ClearAutoIntersectionStatus();

	UE_API bool CanAutoConnect(const UZoneShapeComponent* ShapeComp) const;
	UE_API bool CanAutoCreateIntersection(const UZoneShapeComponent* ShapeComp) const;
};

#undef UE_API
