// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/AssetEditorToolkit.h"


/** ZoneGraph Editor public interface */
class IZoneGraphEditor : public FAssetEditorToolkit
{
public:

};


