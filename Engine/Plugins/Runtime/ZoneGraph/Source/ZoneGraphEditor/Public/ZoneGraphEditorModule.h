// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once

#include "Modules/ModuleInterface.h"
#include "UObject/NameTypes.h"
#include "Templates/SharedPointer.h"

class FComponentVisualizer;
class FUICommandList;

/**
* The public interface to this module
*/
class FZoneGraphEditorModule : public IModuleInterface
{
public:
	// Begin IModuleInterface
	ZONEGRAPHEDITOR_API virtual void StartupModule() override;
	ZONEGRAPHEDITOR_API virtual void ShutdownModule() override;

private:
	ZONEGRAPHEDITOR_API void RegisterComponentVisualizer(FName ComponentClassName, TSharedPtr<FComponentVisualizer> Visualizer);
	ZONEGRAPHEDITOR_API void RegisterMenus();

	ZONEGRAPHEDITOR_API void OnBuildZoneGraph();

	TArray<FName> RegisteredComponentClassNames;
	TSharedPtr<FUICommandList> PluginCommands;
};

