// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ZoneGraphEditor : ModuleRules
	{
		public ZoneGraphEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicIncludePaths.AddRange(
			new string[] {
			}
			);

			PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"AssetTools",
				"EditorFramework",
				"UnrealEd",
				"Slate",
				"SlateCore",
				
				"LevelEditor",
				"PropertyEditor",
				"ZoneGraph",
				"DetailCustomizations",
				"ComponentVisualizers",
			}
			);

			PrivateDependencyModuleNames.AddRange(
			new string[] {
				"RenderCore",
				"GraphEditor",
				"KismetWidgets",
				"PropertyEditor",
				"AIGraph",
				"ToolMenus",
				"AppFramework",
				"Projects",
			}
			);
		}

	}
}
