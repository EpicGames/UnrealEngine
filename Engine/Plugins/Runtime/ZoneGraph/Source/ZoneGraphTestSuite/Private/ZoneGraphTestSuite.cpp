// Copyright Epic Games, Inc. All Rights Reserved.

#include "ZoneGraphTestSuite.h"

#define LOCTEXT_NAMESPACE "ZoneGraphTestSuite"

class FZoneGraphTestSuite : public IZoneGraphTestSuite
{
};

IMPLEMENT_MODULE(FZoneGraphTestSuite, ZoneGraphTestSuite)

#undef LOCTEXT_NAMESPACE
