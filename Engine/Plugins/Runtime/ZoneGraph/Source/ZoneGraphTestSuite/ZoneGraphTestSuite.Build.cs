// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

namespace UnrealBuildTool.Rules
{
	public class ZoneGraphTestSuite : ModuleRules
	{
		public ZoneGraphTestSuite(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

			PublicIncludePaths.AddRange(
				new string[] {
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"AIModule",
					"ZoneGraph",
					"AITestSuite",
				}
			);

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					// ... add any modules that your module loads dynamically here ...
				}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.Add("EditorFramework");
				PrivateDependencyModuleNames.Add("UnrealEd");
			}
		}
	}
}