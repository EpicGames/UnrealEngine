{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "0.5",
	"FriendlyName" : "ZoneGraph",
	"Description" : "Description missing.",
	"Category" : "AI",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "ZoneGraph",
			"Type" : "Runtime",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "ZoneGraphEditor",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit"
		},
		{
			"Name" : "ZoneGraphTestSuite",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "Default"
		},
		{
			"Name": "ZoneGraphDebug",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}